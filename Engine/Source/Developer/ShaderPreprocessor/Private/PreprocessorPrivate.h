// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShaderCore.h"
#include "ShaderCompilerCore.h"
#include "ShaderPreprocessTypes.h"

enum class EMessageType
{
	Error = 0,
	Warn = 1,
	ShaderMetaData = 2,
};

/**
 * Filter preprocessor errors.
 * @param ErrorMsg - The error message.
 * @returns true if the message is valid and has not been filtered out.
 */
inline EMessageType FilterPreprocessorError(const FString& ErrorMsg)
{
	const TCHAR* SubstringsToFilter[] =
	{
		TEXT("Unknown encoding:"),
		TEXT("with no newline, supplemented newline"),
		TEXT("Converted [CR+LF] to [LF]")
	};
	const int32 FilteredSubstringCount = UE_ARRAY_COUNT(SubstringsToFilter);

	if (ErrorMsg.Contains(TEXT("UESHADERMETADATA")))
	{
		return EMessageType::ShaderMetaData;
	}

	for (int32 SubstringIndex = 0; SubstringIndex < FilteredSubstringCount; ++SubstringIndex)
	{
		if (ErrorMsg.Contains(SubstringsToFilter[SubstringIndex]))
		{
			return EMessageType::Warn;
		}
	}
	return EMessageType::Error;
}

static void ExtractDirective(FString& OutString, FString WarningString)
{
	static const FString PrefixString = TEXT("UESHADERMETADATA_");
	uint32 DirectiveStartPosition = WarningString.Find(PrefixString) + PrefixString.Len();
	uint32 DirectiveEndPosition = WarningString.Find(TEXT("\n"));
	if (DirectiveEndPosition == INDEX_NONE)
	{
		DirectiveEndPosition = WarningString.Len();
	}
	OutString = WarningString.Mid(DirectiveStartPosition, (DirectiveEndPosition - DirectiveStartPosition));
}
