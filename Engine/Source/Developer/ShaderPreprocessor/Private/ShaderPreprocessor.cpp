// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderPreprocessor.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "PreprocessorPrivate.h"
#include "ShaderCompilerDefinitions.h"
#include "stb_preprocess/preprocessor.h"
#include "stb_preprocess/stb_alloc.h"
#include "stb_preprocess/stb_ds.h"

static TAutoConsoleVariable<int32> CVarShaderCompilerThreadLocalPreprocessBuffer(
	TEXT("r.ShaderCompiler.ThreadLocalPreprocessBuffer"),
	1280 * 1024,
	TEXT("Amount to preallocate for preprocess output per worker thread, to save reallocation overhead in the preprocessor."),
	ECVF_Default
);

namespace
{
	const FString PlatformHeader = TEXT("/Engine/Public/Platform.ush");
	void LogMandatoryHeaderError(const FShaderCompilerInput& Input, FShaderPreprocessOutput& Output)
	{
		FString Path = Input.VirtualSourceFilePath;
		FString Message = FString::Printf(TEXT("Error: Shader is required to include %s"), *PlatformHeader);
		Output.LogError(MoveTemp(Path), MoveTemp(Message), 1);
	}
}

// Utility function to wrap FShaderPreprocessDependencies hash table lookups -- used with FComparePathInSource / FCompareResultPath below
template <typename CompareType, typename... ArgsType>
FORCEINLINE uint32 DependencyHashTableFind(const FShaderPreprocessDependencies& Dependencies, const CompareType& Compare, uint32 KeyHash, ArgsType... Args)
{
	const FHashTable& HashTable = Compare.GetHashTable(Dependencies);
	for (uint32 Index = HashTable.First(KeyHash); HashTable.IsValid(Index); Index = HashTable.Next(Index))
	{
		if (Compare.Equals(Dependencies.Dependencies[Index], Args...))
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

struct FComparePathInSource
{
	static FORCEINLINE const FHashTable& GetHashTable(const FShaderPreprocessDependencies& Dependencies)
	{
		return Dependencies.BySource;
	}
	static FORCEINLINE bool Equals(const FShaderPreprocessDependency& Dependency, const ANSICHAR* PathInSource, uint32 PathLen, FXxHash64 PathHash, const ANSICHAR* ParentPathAnsi)
	{
		return Dependency.EqualsPathInSource(PathInSource, PathLen, PathHash, ParentPathAnsi);
	}
};

struct FCompareResultPath
{
	static FORCEINLINE const FHashTable& GetHashTable(const FShaderPreprocessDependencies& Dependencies)
	{
		return Dependencies.ByResult;
	}
	static FORCEINLINE bool Equals(const FShaderPreprocessDependency& Dependency, const FString& ResultPath, uint32 ResultPathHash)
	{
		return Dependency.EqualsResultPath(ResultPath, ResultPathHash);
	}
};

static void AddStbDefine(stb_arena* MacroArena, macro_definition**& StbDefines, const TCHAR* Name, const TCHAR* Value);
static void AddStbDefines(stb_arena* MacroArena, macro_definition**& StbDefines, const FShaderCompilerDefinitions& Defines);

class FShaderPreprocessorUtilities
{
public:
	static void PopulateDefines(const FShaderCompilerEnvironment& Environment, const FShaderCompilerDefinitions& AdditionalDefines, stb_arena* MacroArena, macro_definition**& OutDefines)
	{
		arrsetcap(OutDefines, Environment.Definitions->Num() + AdditionalDefines.Num());
		AddStbDefines(MacroArena, OutDefines, *Environment.Definitions);
		AddStbDefines(MacroArena, OutDefines, AdditionalDefines);
	}
};

//////////////////////////////////////////////////////////////////////////
extern "C"
{
	// adapter functions for STB memory allocation
	void* StbMalloc(size_t Size) 
	{
		void* Alloc = FMemory::Malloc(Size);
		return Alloc; 
	}

	void* StbRealloc(void* Pointer, size_t Size) 
	{
		void* Alloc = FMemory::Realloc(Pointer, Size);
		return Alloc;
	}

	void StbFree(void* Pointer) 
	{
		return FMemory::Free(Pointer); 
	}

	ANSICHAR* StbStrDup(const ANSICHAR* InString)
	{
		if (InString)
		{
			int32 Len = FCStringAnsi::Strlen(InString) + 1;
			ANSICHAR* Result = reinterpret_cast<ANSICHAR*>(StbMalloc(Len));
			return FCStringAnsi::Strncpy(Result, InString, Len);
		}
		return nullptr;
	}

	size_t StbLcgNext()
	{
		// thread local linear congruential generator
		static thread_local uint64 Current = 0x31415926;
		static const uint64 A = 0x27BB2EE687B0B0FDULL;
		static const uint64 B = 0xB504F32DULL;
		
		size_t Next = Current;
		Current = Current * A + B;

		return (size_t)Next;
	}

	void StbAssertFailed(const char* File, int Line, const char* Expression, const char* Format, ...)
	{
		ANSICHAR Message[1024];
		Message[0] = '\0';
		if (Format && Format[0])
		{
			va_list Args;
			va_start(Args, Format);
			FCStringAnsi::GetVarArgs(Message, UE_ARRAY_COUNT(Message), Format, Args);
			va_end(Args);
		}
		checkf(false, TEXT("[stb_preprocess] %hs(%d): assert(%hs)%s%hs"),
			File ? File : "<unknown>",
			Line,
			Expression ? Expression : "",
			Message[0] ? TEXT(" -- ") : TEXT(""),
			Message);
	}
}

struct FStbLoadedInclude
{
	const ANSICHAR* FileName = nullptr;			// Points to ResultPath in FShaderPreprocessDependenciesShared, or LocalFileName
	const ANSICHAR* Data = nullptr;				// Points to SharedData, LocalData, or data from FShaderCompilerEnvironment
	size_t DataLength = 0;
	FShaderSharedAnsiStringPtr SharedData;
	TArray<ANSICHAR> LocalData;
	TArray<ANSICHAR> LocalFileName;
};

static bool HasDependencyFromResultPath(const FShaderPreprocessDependencies& Dependencies, const FString& ResultPath, const FStbLoadedInclude* CacheShared);

struct FStbPreprocessContext
{
	const FShaderCompilerInput& ShaderInput;
	const FShaderCompilerEnvironment& Environment;
	TMap<FString, FStbLoadedInclude> LoadedIncludesCache;

	// Shared includes from PreprocessDependencies, VertexFactoryDependencies, and Environment.IncludeVirtualPathToSharedContentsMap
	// are stored in this array instead of the map, indexed sequentially.  Avoids hash table overhead of "LoadedIncludesCache".
	TArray<FStbLoadedInclude> LoadedIncludesCacheShared;
	FShaderPreprocessDependenciesShared PreprocessDependencies;
	FShaderPreprocessDependenciesShared VertexFactoryDependencies;
	FHashTable SharedContentsHash;								// Case insensitive hash table pointing at LoadedIncludesCacheShared with entries from IncludeVirtualPathToSharedContentsMap
	uint32 SharedIncludeIndex = INDEX_NONE;						// Index in LoadedIncludesCacheShared propagated from StbResolveInclude to StbLoadFile
	uint32 VertexFactoryOffset = INDEX_NONE;					// Vertex factory dependencies start at this offset in LoadedIncludesCacheShared
	uint32 VirtualSharedContentsOffset = INDEX_NONE;			// Virtual shared contents start at this offset in LoadedIncludesCacheShared

	// TEXT macro processing state
	struct FTextEntry
	{
		uint32  Index;
		uint32  Hash;
		uint32  Offset;
		bool    bIsAssert;
		FString SourceText;
		FString ConvertedText;
		FString EncodedText;
	};
	TArray<FTextEntry> TextEntries;
	TArray<ANSICHAR> TextMacroSubstituted;
	uint32 TextGlobalCount = 0;
	uint32 TextAssertCount = 0;
	uint32 TextPrintfCount = 0;
	bool bInAssert = false;

	bool HasIncludedHeader(const FString& Header)
	{
		// Checks if a given header  has been included.  Note that the header may be encountered through one of our FShaderPreprocessDependencies structures,
		// so if those are valid, we need to check the corresponding elements in the LoadedIncludesCacheShared array to see if the path was encountered.
		return
			(PreprocessDependencies.IsValid() && HasDependencyFromResultPath(*PreprocessDependencies, Header, &LoadedIncludesCacheShared[0])) ||
			(VertexFactoryDependencies.IsValid() && HasDependencyFromResultPath(*VertexFactoryDependencies, Header, &LoadedIncludesCacheShared[VertexFactoryOffset])) ||
			LoadedIncludesCache.Contains(Header);
	}

	bool HasIncludedMandatoryHeaders()
	{
		// Check if the mandatory PlatformHeader has been included ("/Engine/Public/Platform.ush")
		return HasIncludedHeader(PlatformHeader);
	}

	void ShaderPrintGenerate(char*& PreprocessFile, TArray<FShaderDiagnosticData>* OutDiagnosticDatas);
};

static void CopyStringToAnsiCharArray(const TCHAR* Text, int32 TextLen, TArray<ANSICHAR>& Out)
{
	Out.SetNumUninitialized(TextLen + 1);
	ANSICHAR* OutData = Out.GetData();
	for (int32 CharIndex = 0; CharIndex < TextLen; CharIndex++, OutData++, Text++)
	{
		*OutData = (ANSICHAR)*Text;
	}
	*OutData = 0;
}

// Adds 16 bytes of zeroes at end, to allow SSE reads at the end of the buffer without reading past the end of the heap allocation
static void CopyStringToAnsiCharArraySSEPadded(const TCHAR* Text, int32 TextLen, TArray<ANSICHAR>& Out)
{
	constexpr int32 SSEPadding = 16;
	Out.SetNumUninitialized(TextLen + SSEPadding);
	ANSICHAR* OutData = Out.GetData();
	for (int32 CharIndex = 0; CharIndex < TextLen; CharIndex++, OutData++, Text++)
	{
		*OutData = (ANSICHAR)*Text;
	}
	FMemory::Memset(OutData, 0, SSEPadding * sizeof(ANSICHAR));
}

static void StbLoadedIncludeTrimPadding(FStbLoadedInclude* ContentsCached)
{
	// Need 15 characters beyond null terminator, so an unaligned SSE read at the null terminator can safely read 15 extra unused characters
	// without going out of memory bounds.  ShaderConvertAndStripComments adds this padding in the form of extra trailing zeroes.  Verify the
	// data pointer is valid and the trailing zeroes are there before trimming; any caller that gets past this check is guaranteed to have a
	// non-null, NUL-terminated buffer that the C preprocessor can scan safely.  Callers must populate FileName before calling so the assert
	// message identifies which include misbehaved.
	static const char SixteenZeroes[16] = { 0 };
	checkf(ContentsCached->Data != nullptr && ContentsCached->DataLength >= 16 &&
		memcmp(&ContentsCached->Data[ContentsCached->DataLength - 16], SixteenZeroes, 16) == 0,
		TEXT("Shader preprocessor ANSI source '%hs' is malformed (Data=%p, Length=%llu); requires non-null buffer with 15 bytes of zero padding past null terminator."),
		ContentsCached->FileName ? ContentsCached->FileName : "<unnamed>",
		ContentsCached->Data,
		(unsigned long long)ContentsCached->DataLength);

	ContentsCached->DataLength -= 15;
}

static const ANSICHAR* StbLoadFile(const ANSICHAR* Filename, void* RawContext, size_t* OutLength)
{
	FStbPreprocessContext& Context = *reinterpret_cast<FStbPreprocessContext*>(RawContext);

	// Check if we found this file in our preprocess dependencies (fast path)
	if (Context.SharedIncludeIndex != INDEX_NONE)
	{
		FStbLoadedInclude* ContentsCached = &Context.LoadedIncludesCacheShared[Context.SharedIncludeIndex];

		// Reset this after we consume it (although StbResolveInclude should clear it as well before StbLoadFile is called again)
		Context.SharedIncludeIndex = INDEX_NONE;

		checkf(ContentsCached->Data != nullptr && ContentsCached->DataLength > 0,
			TEXT("StbLoadFile: shared include '%hs' (requested as '%hs') has invalid data (Data=%p, Length=%llu)."),
			ContentsCached->FileName ? ContentsCached->FileName : "<unnamed>",
			Filename ? Filename : "<null>",
			ContentsCached->Data, (unsigned long long)ContentsCached->DataLength);

		*OutLength = ContentsCached->DataLength;
		return ContentsCached->Data;
	}

	FString FilenameConverted = StringCast<TCHAR>(Filename).Get();
	uint32 FilenameConvertedHash = GetTypeHash(FilenameConverted);

	FStbLoadedInclude& ContentsCached = Context.LoadedIncludesCache.FindOrAddByHash(FilenameConvertedHash, FilenameConverted);
	if (!ContentsCached.Data)
	{
		const FString* InMemorySource = Context.Environment.IncludeVirtualPathToContentsMap.FindByHash(FilenameConvertedHash, FilenameConverted);

		if (InMemorySource)
		{
			check(!InMemorySource->IsEmpty());
			ShaderConvertAndStripComments(*InMemorySource, ContentsCached.LocalData);

			ContentsCached.Data = ContentsCached.LocalData.GetData();
			ContentsCached.DataLength = ContentsCached.LocalData.Num();
		}
		else
		{
			const FThreadSafeSharedAnsiStringPtr* InMemorySourceAnsi = Context.Environment.IncludeVirtualPathToSharedContentsMap.FindByHash(FilenameConvertedHash, FilenameConverted);

			if (InMemorySourceAnsi)
			{
				ContentsCached.Data = InMemorySourceAnsi->Get()->GetData();
				ContentsCached.DataLength = InMemorySourceAnsi->Get()->Num();
			}
			else
			{
				CheckShaderHashCacheInclude(FilenameConverted, Context.ShaderInput.Target.GetPlatform(), Context.ShaderInput.ShaderFormat.ToString());
				LoadShaderSourceFile(*FilenameConverted, Context.ShaderInput.Target.GetPlatform(), nullptr, nullptr, nullptr, &ContentsCached.SharedData);

				ContentsCached.Data = ContentsCached.SharedData->GetData();
				ContentsCached.DataLength = ContentsCached.SharedData->Num();
			}
		}

		// Populate the ANSI filename for diagnostic identification by the TrimPadding assert (and any later use).
		CopyStringToAnsiCharArray(&FilenameConverted[0], FilenameConverted.Len(), ContentsCached.LocalFileName);
		ContentsCached.FileName = ContentsCached.LocalFileName.GetData();
		StbLoadedIncludeTrimPadding(&ContentsCached);
	}

	// Defense-in-depth: even cache hits must yield a non-null buffer, since downstream C preprocessor code dereferences it unconditionally.
	checkf(ContentsCached.Data != nullptr && ContentsCached.DataLength > 0,
		TEXT("StbLoadFile: include '%s' resolved to invalid data (Data=%p, Length=%llu)."),
		*FilenameConverted, ContentsCached.Data, (unsigned long long)ContentsCached.DataLength);

	*OutLength = ContentsCached.DataLength;
	return ContentsCached.Data;
}

static void StbFreeFile(const ANSICHAR* Filename, const ANSICHAR* Contents, void* RawContext)
{
	// No-op; stripped/converted shader source will be freed from the cache in FStbPreprocessContext when it's destructed;
	// we want to keep it around until that point in case includes are loaded multiple times from different source locations
}

static uint32 ResolveDependencyFromPathInSource(const FShaderPreprocessDependencies& Dependencies, const ANSICHAR* PathInSource, uint32 PathLen, FXxHash64 PathHash, const ANSICHAR* ParentPathAnsi, FStbLoadedInclude* CacheShared)
{
	uint32 HashIndex = DependencyHashTableFind(Dependencies, FComparePathInSource(), GetTypeHash(PathHash), PathInSource, PathLen, PathHash, ParentPathAnsi);
	if (HashIndex != INDEX_NONE)
	{
		// Choose the first unique instance of this result path
		HashIndex = Dependencies.Dependencies[HashIndex].ResultPathUniqueIndex;

		const FShaderPreprocessDependency& Dependency = Dependencies.Dependencies[HashIndex];

		FStbLoadedInclude* ContentsCached = &CacheShared[HashIndex];
		if (!ContentsCached->FileName)
		{
			ContentsCached->FileName = Dependency.ResultPath.GetData();
			ContentsCached->Data = Dependency.StrippedSource->GetData();
			ContentsCached->DataLength = Dependency.StrippedSource->Num();
			StbLoadedIncludeTrimPadding(ContentsCached);
		}
	}
	return HashIndex;
}

static uint32 ResolveDependencyFromResultPath(const FShaderPreprocessDependencies& Dependencies, const FString& ResultPath, uint32 ResultPathHash, FStbLoadedInclude* CacheShared)
{
	// ResultPathHash is passed in twice -- once for "Find" function, and again as an argument to the "FCompareResultPath::Equals" function
	uint32 HashIndex = DependencyHashTableFind(Dependencies, FCompareResultPath(), ResultPathHash, ResultPath, ResultPathHash);
	if (HashIndex != INDEX_NONE)
	{
		const FShaderPreprocessDependency& Dependency = Dependencies.Dependencies[HashIndex];

		FStbLoadedInclude* ContentsCached = &CacheShared[HashIndex];
		if (!ContentsCached->FileName)
		{
			ContentsCached->FileName = Dependency.ResultPath.GetData();
			ContentsCached->Data = Dependency.StrippedSource->GetData();
			ContentsCached->DataLength = Dependency.StrippedSource->Num();
			StbLoadedIncludeTrimPadding(ContentsCached);
		}
	}
	return HashIndex;
}

// Returns true if the path in question was encountered during preprocessing, if the path is one of the paths referenced by that dependency structure.
static bool HasDependencyFromResultPath(const FShaderPreprocessDependencies& Dependencies, const FString& ResultPath, const FStbLoadedInclude* CacheShared)
{
	uint32 ResultPathHash = GetTypeHash(ResultPath);
	uint32 HashIndex = DependencyHashTableFind(Dependencies, FCompareResultPath(), ResultPathHash, ResultPath, ResultPathHash);

	// Entry will have FileName set if it was encountered
	return HashIndex != INDEX_NONE && CacheShared[HashIndex].FileName != nullptr;
}

static const ANSICHAR* StbResolveInclude(const ANSICHAR* PathInSource, uint32 PathLen, const ANSICHAR* ParentPathAnsi, void* RawContext)
{
	FStbPreprocessContext& Context = *reinterpret_cast<FStbPreprocessContext*>(RawContext);

	FXxHash64 PathHash = FXxHash64::HashBuffer(PathInSource, PathLen);

	// Try main shader preprocess dependencies
	Context.SharedIncludeIndex = INDEX_NONE;
	if (Context.PreprocessDependencies.IsValid())
	{
		uint32 DependencyIndex = ResolveDependencyFromPathInSource(*Context.PreprocessDependencies, PathInSource, PathLen, PathHash, ParentPathAnsi, &Context.LoadedIncludesCacheShared[0]);

		if (DependencyIndex != INDEX_NONE)
		{
			// Propagate the found index to StbLoadFile
			uint32 SharedIncludeIndex = DependencyIndex;
			Context.SharedIncludeIndex = SharedIncludeIndex;
			return Context.LoadedIncludesCacheShared[SharedIncludeIndex].FileName;
		}
	}

	// Try vertex factory preprocess dependencies
	if (Context.VertexFactoryDependencies.IsValid())
	{
		uint32 DependencyIndex = ResolveDependencyFromPathInSource(*Context.VertexFactoryDependencies, PathInSource, PathLen, PathHash, ParentPathAnsi, &Context.LoadedIncludesCacheShared[Context.VertexFactoryOffset]);

		if (DependencyIndex != INDEX_NONE)
		{
			// Propagate the found index to StbLoadFile
			uint32 SharedIncludeIndex = DependencyIndex + Context.VertexFactoryOffset;
			Context.SharedIncludeIndex = SharedIncludeIndex;
			return Context.LoadedIncludesCacheShared[SharedIncludeIndex].FileName;
		}
	}

	// Try SharedContentsHash
	FAnsiStringView RawPathInSourceView(PathInSource, PathLen);
	for (uint32 HashIndex = Context.SharedContentsHash.First(GetTypeHash(RawPathInSourceView)); Context.SharedContentsHash.IsValid(HashIndex); HashIndex = Context.SharedContentsHash.Next(HashIndex))
	{
		if (RawPathInSourceView == Context.LoadedIncludesCacheShared[HashIndex].FileName)
		{
			// Propagate the found index to StbLoadFile
			Context.SharedIncludeIndex = HashIndex;
			return Context.LoadedIncludesCacheShared[HashIndex].FileName;
		}
	}

	// Slow path...  Platform specific files and procedurally generated files (/Engine/Generated/Material.ush) -- typically 5% of files.
	FString PathModified = FString::ConstructFromPtrSize(PathInSource, PathLen);
	if (!PathModified.StartsWith(TEXT("/"))) // if path doesn't start with / it's relative, if so append the parent's folder and collapse any relative dirs
	{
		FString ParentFolder(ParentPathAnsi);
		ParentFolder = FPaths::GetPath(ParentFolder);
		PathModified = ParentFolder / PathModified;
		FPaths::CollapseRelativeDirectories(PathModified);
	}

	FixupShaderFilePath(PathModified, Context.ShaderInput.Target.GetPlatform(), &Context.ShaderInput.ShaderPlatformName);
	uint32 PathModifiedHash = GetTypeHash(PathModified);

	// We need to check our preprocess dependencies again with the result path, so we get the canonical capitalization for it from the dependencies, if available.
	// This case can be reached for platform includes (which aren't added to the bulk dependencies).
	if (Context.PreprocessDependencies.IsValid())
	{
		uint32 DependencyIndex = ResolveDependencyFromResultPath(*Context.PreprocessDependencies, PathModified, PathModifiedHash, &Context.LoadedIncludesCacheShared[0]);

		if (DependencyIndex != INDEX_NONE)
		{
			// Propagate the found index to StbLoadFile
			uint32 SharedIncludeIndex = DependencyIndex;
			Context.SharedIncludeIndex = SharedIncludeIndex;
			return Context.LoadedIncludesCacheShared[SharedIncludeIndex].FileName;
		}
	}

	// Try vertex factory preprocess dependencies
	if (Context.VertexFactoryDependencies.IsValid())
	{
		uint32 DependencyIndex = ResolveDependencyFromResultPath(*Context.VertexFactoryDependencies, PathModified, PathModifiedHash, &Context.LoadedIncludesCacheShared[Context.VertexFactoryOffset]);

		if (DependencyIndex != INDEX_NONE)
		{
			// Propagate the found index to StbLoadFile
			uint32 SharedIncludeIndex = DependencyIndex + Context.VertexFactoryOffset;
			Context.SharedIncludeIndex = SharedIncludeIndex;
			return Context.LoadedIncludesCacheShared[SharedIncludeIndex].FileName;
		}
	}

	// If we reach here, the include will be added to the map.  Check if it's already in the map.
	FStbLoadedInclude* ContentsCached = Context.LoadedIncludesCache.FindByHash(PathModifiedHash, PathModified);
	if (ContentsCached)
	{
		// We return the same previously resolved path so preprocessor will handle #pragma once with files included with inconsistent casing correctly
		return ContentsCached->FileName;
	}

	bool bExists =
		Context.Environment.IncludeVirtualPathToContentsMap.ContainsByHash(PathModifiedHash, PathModified) ||
		// LoadShaderSourceFile will load the file if it exists, but then cache it internally, so the next call in StbLoadFile will be cheap
		// (and hence this is not wasteful, just performs the loading earlier)
		LoadShaderSourceFile(*PathModified, Context.ShaderInput.Target.GetPlatform(), nullptr, nullptr);

	if (bExists)
	{
		ContentsCached = &Context.LoadedIncludesCache.AddByHash(PathModifiedHash, PathModified);

		// Initialize the ANSI file name in the map entry.  The file itself will be loaded in StbLoadFile, but we need the ANSI string
		// as the return value from this function. 
		CopyStringToAnsiCharArray(&PathModified[0], PathModified.Len(), ContentsCached->LocalFileName);
		ContentsCached->FileName = ContentsCached->LocalFileName.GetData();
		return ContentsCached->FileName;
	}

	return nullptr;
}

static const char* ShaderPrintTextIdentifier = "TEXT";
static const char* ShaderPrintAssertIdentifier = "UEReportAssertWithPayload";

static const char* StbCustomMacroBegin(const char* OriginalText, void* RawContext)
{
	FStbPreprocessContext& Context = *reinterpret_cast<FStbPreprocessContext*>(RawContext);

	// Check for assert macro
	if (FCStringAnsi::Strstr(OriginalText, ShaderPrintAssertIdentifier) == OriginalText)
	{
		// We only need to track that we're in an assert, we don't need to do any substitution
		Context.bInAssert = true;
		return OriginalText;
	}

	// TEXT macro
	check(FCStringAnsi::Strstr(OriginalText, ShaderPrintTextIdentifier) == OriginalText);
	const char* TextChar = OriginalText;
	while (*TextChar != '(')
	{
		TextChar++;
	}
	TextChar++;
	while (*TextChar != ')' && *TextChar != '\"')
	{
		TextChar++;
	}

	// If no quoted text, that's a parse error
	if (*TextChar != '\"')
	{
		return nullptr;
	}

	// We found a string, add an entry
	const uint32 EntryIndex = Context.TextEntries.Num();
	FStbPreprocessContext::FTextEntry& Entry = Context.TextEntries.AddDefaulted_GetRef();
	Entry.Index = EntryIndex;
	Entry.Offset = Context.TextGlobalCount;
	Entry.bIsAssert = Context.bInAssert;
	if (Entry.bIsAssert)
	{
		++Context.TextAssertCount;
	}
	else
	{
		++Context.TextPrintfCount;
	}
	
	// Parse the string, handling escaped characters.  SourceText contains the raw text, ConvertedText removes escape back slashes,
	// and EncodedText is an array of integer numeric values as ASCII.
	TextChar++;
	const char* TextStart = TextChar;
	int32 CharCount = 0;
	for (; *TextChar != '\"'; TextChar++)
	{
		if (*TextChar == '\\')
		{
			TextChar++;
		}
		CharCount++;
	}
	Entry.SourceText = FString(FAnsiStringView(TextStart, TextChar - TextStart));
	Entry.ConvertedText.GetCharArray().SetNumUninitialized(CharCount + 1);
	Entry.EncodedText.Reserve(CharCount * 4); // ~3 digits per character + a comma

	TCHAR* ConvertedTextData = Entry.ConvertedText.GetCharArray().GetData();

	int32 CharIndex = 0;
	for (TextChar = TextStart; *TextChar != '\"'; TextChar++, CharIndex++)
	{
		if (*TextChar == '\\')
		{
			TextChar++;
		}
		ConvertedTextData[CharIndex] = *TextChar;

		const char C = *TextChar;
		Entry.EncodedText.AppendInt(uint8(C));
		if (CharIndex + 1 != CharCount)
		{
			Entry.EncodedText += ',';
		}
	}
	check(CharIndex == CharCount);
	ConvertedTextData[CharIndex] = 0;

	Entry.Hash = CityHash32((const char*)Entry.SourceText.GetCharArray().GetData(), sizeof(FString::ElementType) * Entry.SourceText.Len());
	Context.TextGlobalCount += Entry.ConvertedText.Len();

	// Generate substitution string -- need SSE padding on any text handled by the preprocessor
	if (Entry.bIsAssert)
	{
		const FString HashString = FString::Printf(TEXT("%u"), Entry.Hash);
		CopyStringToAnsiCharArraySSEPadded(*HashString, HashString.Len(), Context.TextMacroSubstituted);
	}
	else
	{
		const FString InitHashBegin(TEXT("InitShaderPrintText("));
		const FString InitHashEnd(TEXT(")"));

		const FString HashText = InitHashBegin + FString::FromInt(EntryIndex) + InitHashEnd;
		CopyStringToAnsiCharArraySSEPadded(*HashText, HashText.Len(), Context.TextMacroSubstituted);
	}

	return Context.TextMacroSubstituted.GetData();
}

static void StbCustomMacroEnd(const char* OriginalText, void* RawContext, const char* SubstitutionText)
{
	FStbPreprocessContext& Context = *reinterpret_cast<FStbPreprocessContext*>(RawContext);

	if (FCStringAnsi::Strstr(OriginalText, ShaderPrintAssertIdentifier) == OriginalText)
	{
		Context.bInAssert = false;
	}
}

void FStbPreprocessContext::ShaderPrintGenerate(char*& PreprocessedFile, TArray<FShaderDiagnosticData>* OutDiagnosticDatas)
{
	// Check if ShaderPrintCommon.ush was included, to decide whether to add the shader print generated code
	static FString ShaderPrintHeader("/Engine/Private/ShaderPrintCommon.ush");
	if (!HasIncludedHeader(ShaderPrintHeader))
	{
		return;
	}

	// 1. Write a global struct containing all the entries
	// 2. Write the function for fetching character for a given entry index
	const uint32 EntryCount = TextEntries.Num();
	FString TextChars;
	if (TextPrintfCount > 0 && EntryCount > 0 && TextGlobalCount > 0)
	{
		// 1. Encoded character for each text entry within a single global char array
		TextChars = FString::Printf(TEXT("\n\nstatic const uint TEXT_CHARS[%d] = {\n"), TextGlobalCount);
		for (FTextEntry& Entry : TextEntries)
		{
			TextChars += FString::Printf(TEXT("\t%s%s // %d: \"%s\"\n"), *Entry.EncodedText, Entry.Index < EntryCount - 1 ? TEXT(",") : TEXT(""), Entry.Index, *Entry.SourceText);
		}
		TextChars += TEXT("};\n\n");

		// 2. Offset within the global array
		TextChars += FString::Printf(TEXT("static const uint TEXT_OFFSETS[%d] = {\n"), EntryCount + 1);
		for (FTextEntry& Entry : TextEntries)
		{
			TextChars += FString::Printf(TEXT("\t%d, // %d: \"%s\"\n"), Entry.Offset, Entry.Index, *Entry.SourceText);
		}
		TextChars += FString::Printf(TEXT("\t%d // end\n"), TextGlobalCount);
		TextChars += TEXT("};\n\n");

		// 3. Entry hashes
		TextChars += TEXT("// Hashes are computed using the CityHash32 function\n");
		TextChars += FString::Printf(TEXT("static const uint TEXT_HASHES[%d] = {\n"), EntryCount);
		for (FTextEntry& Entry : TextEntries)
		{
			TextChars += FString::Printf(TEXT("\t0x%x%s // %d: \"%s\"\n"), Entry.Hash, Entry.Index < EntryCount - 1 ? TEXT(",") : TEXT(""), Entry.Index, *Entry.SourceText);
		}
		TextChars += TEXT("};\n\n");

		TextChars += TEXT("uint ShaderPrintGetChar(uint InIndex)              { return TEXT_CHARS[InIndex]; }\n");
		TextChars += TEXT("uint ShaderPrintGetOffset(FShaderPrintText InText) { return TEXT_OFFSETS[InText.Index]; }\n");
		TextChars += TEXT("uint ShaderPrintGetHash(FShaderPrintText InText)   { return TEXT_HASHES[InText.Index]; }\n");
	}
	else
	{
		TextChars += TEXT("uint ShaderPrintGetChar(uint Index)                { return 0; }\n");
		TextChars += TEXT("uint ShaderPrintGetOffset(FShaderPrintText InText) { return 0; }\n");
		TextChars += TEXT("uint ShaderPrintGetHash(FShaderPrintText InText)   { return 0; }\n");
	}

	// 3. Insert global struct data + print function
	TArray<ANSICHAR> TextCharsAnsi;
	CopyStringToAnsiCharArray(*TextChars, TextChars.Len(), TextCharsAnsi);

	PreprocessedFile = preprocessor_file_append(PreprocessedFile, TextCharsAnsi.GetData(), TextCharsAnsi.Num() - 1);

	// 4. Insert assert data into shader compilation output for runtime CPU lookup
	if (OutDiagnosticDatas && TextAssertCount > 0)
	{
		OutDiagnosticDatas->Reserve(OutDiagnosticDatas->Num() + TextAssertCount);
		for (const FTextEntry& E : TextEntries)
		{
			if (E.bIsAssert)
			{
				FShaderDiagnosticData& Data = OutDiagnosticDatas->AddDefaulted_GetRef();
				Data.Hash = E.Hash;
				Data.Message = E.SourceText;
			}
		}
	}
}

class FShaderPreprocessorModule : public IModuleInterface
{
	virtual void StartupModule() override
	{
		init_preprocessor(&StbLoadFile, &StbFreeFile, &StbResolveInclude, &StbCustomMacroBegin, &StbCustomMacroEnd);
		// disable the "directive not at start of line" error; this allows a few things:
		// 1. #define'ing #pragma messages - consumed by the preprocessor (to handle UESHADERMETADATA hackery)
		// 2. #define'ing other #pragmas (those not processed explicitly by the preprocessor are copied into the preprocessed code
		// 3. handling the HLSL infinity constant (1.#INF); STB preprocessor interprets any use of # as a directive which is not the case here
		pp_set_warning_mode(PP_RESULT_directive_not_at_start_of_line, PP_RESULT_MODE_no_warning); 
	}
};
IMPLEMENT_MODULE(FShaderPreprocessorModule, ShaderPreprocessor);

static void AddStbDefine(stb_arena* MacroArena, macro_definition**& StbDefines, const TCHAR* Name, const TCHAR* Value)
{
	TAnsiStringBuilder<256> Define;

	// Define format:  "%s %s"  (Name Value)
	Define.Append(Name);
	Define.AppendChar(' ');
	Define.Append(Value);

	arrput(StbDefines, pp_define(MacroArena, *Define));
}

static void AddStbDefines(stb_arena* MacroArena, macro_definition**& StbDefines, const FShaderCompilerDefinitions& Defines)
{
	for (FShaderCompilerDefinitions::FConstIterator It(Defines); It; ++It)
	{
		AddStbDefine(MacroArena, StbDefines, It.Key(), It.Value());
	}
}

/**
 * Preprocess a shader.
 * @param OutPreprocessedShader - Upon return contains the preprocessed source code.
 * @param ShaderOutput - ShaderOutput to which errors can be added.
 * @param ShaderInput - The shader compiler input.
 * @param AdditionalDefines - Additional defines with which to preprocess the shader.
 * @param DefinesPolicy - Whether to add shader definitions as comments.
 * @returns true if the shader is preprocessed without error.
 */
bool PreprocessShader(
	FShaderPreprocessOutput& Output,
	const FShaderCompilerInput& Input,
	const FShaderCompilerEnvironment& Environment,
	const FShaderCompilerDefinitions& AdditionalDefines
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PreprocessShader);

	stb_arena MacroArena = { 0 };
	macro_definition** StbDefines = nullptr;
	
	FShaderPreprocessorUtilities::PopulateDefines(Environment, AdditionalDefines, &MacroArena, StbDefines);

	// The substitution text generated by custom macros gets run through the preprocessor afterwards, but in some cases we want to
	// run the arguments through the preprocessor before as well.  The TEXT macro needs this to handle things like TEXT(__FILE__),
	// where the __FILE__ macro needs to be expanded before the custom macro handler is called, so we pass "1" to enable running
	// the preprocessor first.  By contrast, for shader asserts, we must NOT run the preprocessor on the arguments first, because
	// the assert macro sets a state flag which modifies behavior of TEXT macros inside the assert.  Asserts store their TEXT tokens
	// outside the shader for printing in code when an assert is triggered, while ShaderPrint stores TEXT in the shader itself.
	arrput(StbDefines, pp_define_custom_macro(&MacroArena, ShaderPrintTextIdentifier, 1));
	arrput(StbDefines, pp_define_custom_macro(&MacroArena, ShaderPrintAssertIdentifier, 0));

	FStbPreprocessContext Context{ Input, Environment };

	auto InFilename = StringCast<ANSICHAR>(*Input.VirtualSourceFilePath);
	int NumDiagnostics = 0;
	pp_diagnostic* Diagnostics = nullptr;

	static const int32 ThreadLocalPreprocessBufferSize = CVarShaderCompilerThreadLocalPreprocessBuffer.GetValueOnAnyThread();
	static thread_local char* ThreadLocalPreprocessBuffer = nullptr;

	// Sanity check the buffer size so it won't OOM if a bad value is entered.
	int32 ClampedPreprocessBufferSize = ThreadLocalPreprocessBufferSize ? FMath::Clamp(ThreadLocalPreprocessBufferSize, 64 * 1024, 4 * 1024 * 1024) : 0;
	if (ClampedPreprocessBufferSize && !ThreadLocalPreprocessBuffer)
	{
		ThreadLocalPreprocessBuffer = new char[ClampedPreprocessBufferSize];
	}

	if (GetShaderPreprocessDependencies(*Input.VirtualSourceFilePath, Context.ShaderInput.Target.GetPlatform(), Context.PreprocessDependencies))
	{
		// First item in dependencies is always root file, so set that index
		Context.SharedIncludeIndex = 0;
	}

	// Grab vertex factory dependencies if present
	const FString* VertexFactoryInclude = Context.Environment.IncludeVirtualPathToContentsMap.Find(TEXT("/Engine/Generated/VertexFactory.ush"));
	if (VertexFactoryInclude)
	{
		int32 VertexFactoryNameStart;
		int32 VertexFactoryNameEnd;
		if (VertexFactoryInclude->FindChar(TEXT('\"'), VertexFactoryNameStart) && VertexFactoryInclude->FindLastChar(TEXT('\"'), VertexFactoryNameEnd))
		{
			// Should have at least one character in our filename
			check(VertexFactoryNameEnd > VertexFactoryNameStart + 1);
			FString VertexFactoryFilename(FStringView(&(*VertexFactoryInclude)[VertexFactoryNameStart + 1], VertexFactoryNameEnd - (VertexFactoryNameStart + 1)));
			GetShaderPreprocessDependencies(*VertexFactoryFilename, Context.ShaderInput.Target.GetPlatform(), Context.VertexFactoryDependencies);
		}
	}

	// Initialize array of loaded includes associated with PreprocessDependencies, VertexFactoryDependencies, and Environment.IncludeVirtualPathToSharedContentsMap
	Context.VertexFactoryOffset = Context.PreprocessDependencies.IsValid() ? Context.PreprocessDependencies->Dependencies.Num() : 0;
	Context.VirtualSharedContentsOffset = Context.VertexFactoryOffset + (Context.VertexFactoryDependencies.IsValid() ? Context.VertexFactoryDependencies->Dependencies.Num() : 0);
	Context.LoadedIncludesCacheShared.AddDefaulted(Context.VirtualSharedContentsOffset + Context.Environment.IncludeVirtualPathToSharedContentsMap.Num());

	// Initialize root file dependency, if present
	if (Context.PreprocessDependencies.IsValid())
	{
		const FShaderPreprocessDependency& Dependency = Context.PreprocessDependencies->Dependencies[0];
		FStbLoadedInclude* ContentsCached = &Context.LoadedIncludesCacheShared[0];

		ContentsCached->FileName = InFilename.Get();
		ContentsCached->Data = Dependency.StrippedSource->GetData();
		ContentsCached->DataLength = Dependency.StrippedSource->Num();
		StbLoadedIncludeTrimPadding(ContentsCached);
	}

	// Initialize loaded includes for IncludeVirtualPathToSharedContentsMap, and generate a hash table
	uint32 SharedContentsMapIndex = Context.VirtualSharedContentsOffset;
	for (const auto& SharedContentsMapIt : Context.Environment.IncludeVirtualPathToSharedContentsMap)
	{
		FStbLoadedInclude& Include = Context.LoadedIncludesCacheShared[SharedContentsMapIndex];

		// Copy name
		CopyStringToAnsiCharArray(&SharedContentsMapIt.Key[0], SharedContentsMapIt.Key.Len(), Include.LocalFileName);
		Include.FileName = Include.LocalFileName.GetData();

		// Set data
		Include.Data = SharedContentsMapIt.Value->GetData();
		Include.DataLength = SharedContentsMapIt.Value->Num();
		StbLoadedIncludeTrimPadding(&Include);

		// Add to hash table -- GetTypeHash on string view is case insensitive
		Context.SharedContentsHash.Add(GetTypeHash(FAnsiStringView(Include.LocalFileName.GetData(), Include.LocalFileName.Num() - 1)), SharedContentsMapIndex);

		SharedContentsMapIndex++;
	}

	char* OutPreprocessedAnsi = preprocess_file(InFilename.Get(), &Context, StbDefines, arrlen(StbDefines), &Diagnostics, &NumDiagnostics, ThreadLocalPreprocessBuffer, ClampedPreprocessBufferSize);

	bool HasError = false;
	if (Diagnostics != nullptr)
	{
		for (int DiagIndex = 0; DiagIndex < NumDiagnostics; ++DiagIndex)
		{
			pp_diagnostic* Diagnostic = &Diagnostics[DiagIndex];
			HasError |= (Diagnostic->error_level == PP_RESULT_MODE_error);
			
			FString Message = Diagnostic->message;
			if (Diagnostic->error_level == PP_RESULT_MODE_error || Diagnostic->error_level == PP_RESULT_MODE_warning)
			{
				FString Filename = Diagnostic->where->filename;
				Output.LogError(MoveTemp(Filename), MoveTemp(Message), Diagnostic->where->line_number);
			}
			else
			{
				EMessageType Type = FilterPreprocessorError(Message);
				if (Type == EMessageType::ShaderMetaData)
				{
					FString Directive;
					ExtractDirective(Directive, Message);
					Output.AddDirective(MoveTemp(Directive));
				}
			}
		}
	}

	if (!HasError)
	{
		// Append ShaderPrint generated code at the end of the shader if necessary
		Context.ShaderPrintGenerate(OutPreprocessedAnsi, &Output.EditDiagnosticDatas());

		// "preprocessor_file_size" includes null terminator, so subtract one when initializing the FShaderSource (which automatically null terminates)
		Output.EditSource().Set({ OutPreprocessedAnsi, preprocessor_file_size(OutPreprocessedAnsi) - 1 });
	}

	if (!HasError && !Context.HasIncludedMandatoryHeaders())
	{
		LogMandatoryHeaderError(Input, Output);
		HasError = true;
	}

	preprocessor_file_free(OutPreprocessedAnsi, Diagnostics);
	stbds_arrfree(StbDefines);
	stb_arena_free(&MacroArena);

	return !HasError;
}

bool PreprocessShader(
	FShaderPreprocessOutput& Output,
	const FShaderCompilerInput& Input,
	const FShaderCompilerEnvironment& MergedEnvironment)
{
	// overload instead of defaulting FShaderCompilerDefinitions arg to avoid including an internal header in a public header.
	return PreprocessShader(Output, Input, MergedEnvironment, FShaderCompilerDefinitions());
}