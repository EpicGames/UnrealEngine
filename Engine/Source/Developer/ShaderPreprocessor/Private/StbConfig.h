// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <stddef.h>

#ifdef __cplusplus
extern "C"
{
#endif
void* StbMalloc(size_t Size);
void* StbRealloc(void* Pointer, size_t Size);
void StbFree(void* Pointer);
char* StbStrDup(const char* String);
size_t StbLcgNext();
void StbAssertFailed(const char* File, int Line, const char* Expression, const char* Format, ...);
#ifdef __cplusplus
}
#endif

#define STB_COMMON_MALLOC(Size) StbMalloc(Size)
#define STB_COMMON_REALLOC(Pointer, Size) StbRealloc(Pointer, Size)
#define STB_COMMON_FREE(Pointer) StbFree(Pointer)
#define STB_COMMON_STRDUP(String) StbStrDup(String)
#define STB_LCG_NEXT() StbLcgNext()

#define STB_COMMON_ASSERT(expr) \
	do { if (!(expr)) { StbAssertFailed(__FILE__, __LINE__, #expr, ""); } } while (0)
#define STB_COMMON_ASSERTF(expr, fmt, ...) \
	do { if (!(expr)) { StbAssertFailed(__FILE__, __LINE__, #expr, fmt, ##__VA_ARGS__); } } while (0)
