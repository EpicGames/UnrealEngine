// Copyright Epic Games, Inc. All Rights Reserved.

//////////////////////////////////////////////////////////////////////////////
//
// integer constant expression parse & evaluation for #if
//
// The text has already been macro-substituted, and defined() was turned
// into 0 or 1.

#include <ctype.h>
#include <errno.h>
#include <limits.h>
#include <stddef.h>
#include <stdlib.h>
#include <string.h>
#include "cond_expr.h"

#ifdef _MSC_VER
#define intmax_t __int64
#else
#define intmax_t long long
#endif

#if !defined(FORCE_INLINE)
#if defined(_MSC_VER)
#define FORCE_INLINE __forceinline
#else
#define FORCE_INLINE inline __attribute__((always_inline))
#endif
#endif

enum
{
	PPLEX_eof = 0,
	PPLEX_int_literal,
	PPLEX_and_and,
	PPLEX_or_or,
	PPLEX_less_eq,
	PPLEX_greater_eq,
	PPLEX_bang_eq,
	PPLEX_eq_eq,
	PPLEX_shift_left,
	PPLEX_shift_right
};

typedef struct
{
	char* p;
	int token;
	int token_value;
	int result;
} ppcexp;  // preprocessor constant expression lexer

// Faster inline version of CRT isspace function
static FORCE_INLINE unsigned isspace_inline(unsigned p)
{
	// Subtract one so ASCII space (32) fits in the last bit of a 32-bit mask.  The null terminator (zero) will wrap around due to
	// unsigned math, which is good, because it will pass this conditional, avoiding the bit test.
	p = p - 1;
	if (p >= 32)
	{
		return 0;
	}

	// Only invalid non-printable characters (which lead to downstream syntax errors) will fail the bit test, since nullptr is covered
	// by the branch above, so we expect this second branch condition to be 100% correctly predicted.
	static const unsigned whitespace_mask =
		(1u << ('\t' - 1)) |
		(1u << ('\n' - 1)) |
		(1u << ('\v' - 1)) |
		(1u << ('\r' - 1)) |
		(1u << (' '  - 1));

	return whitespace_mask & (1u << p);
}

// Assuming we found a starting digit, this function checks the next character to see if it ends the number.  If so, it's a single
// digit number (which is very common in the preprocessor -- most values are zero or one), and we can do a trivial digit character
// to number conversion, rather than calling the expensive strtoll. This doesn't try to catch all cases, but it catches most
// cases likely to exist in valid (non syntax error) code.
static FORCE_INLINE int is_end_of_number(int p)
{
	// Any ASCII less than a decimal character ('.') ends a number, which includes whitespace, parentheses, some operator
	// characters, and NULL.  Numbers can have hex or binary prefixes (0x, 0b), or unsigned / length suffixes (ull), so we need
	// to allow letters as a possibility, in addition to digits (other letters produce syntax errors, but it's fine if those
	// return false and go through the slow path).
	// 
	// That leaves other operator characters we need to treat as ending the number:  ":<=>?^|~".  Looking at an ASCII chart shows
	// a way to handle those.  All the above characters are at the bottom of the chart, and can be detected by looking at the
	// bottom 5 bits of the character code:
	// 
	//    0x38 8  0x58 X  0x78  x
	//    0x39 9  0x59 Y  0x79  y
	//    0x3a :  0x5a Z  0x7a  z
	//    0x3b ;  0x5b [  0x7b  {
	//    0x3c <  0x5c \  0x7c  |
	//    0x3d =  0x5d ]  0x7d  }
	//    0x3e >  0x5e ^  0x7e  ~
	//    0x3f ?  0x5f _  0x7f  DEL
	//
	// Anything 0x1a or larger in the bottom 5 bits is past the range of possible number continuations.  There's a handful of
	// characters not caught by this logic:  "/@`", but two of those will produce syntax errors, and integer divide is rare in
	// preprocessor expressions.  To avoid two branches, we merge two expressions which will have a negative sign for the
	// condition we care about.  This is equivalent to:
	//
	//    (p < '.') || (0x1a <= (p & 0x1f))
	//
	return ((p - '.') | (0x19 - (p & 0x1f))) < 0;
}

static FORCE_INLINE int is_token_character(unsigned char p)
{
	// Numbers, letteres, underscore, and '$' are considered valid token characters by the preprocessor.  Mask test to see
	// if this is one of those.
	static const unsigned token_mask[8] = { 0x00000000, 0x03ff0010, 0x87fffffe, 0x07fffffe,  0,0,0,0 };

	return (1u << (p & 0x1f)) & token_mask[p >> 5];
}

static void ce_next(ppcexp* c)
{
	char* p = c->p;
	while (isspace_inline(*p))
		++p;

	switch (*p)
	{
		case '<':
			if (p[1] == '<')
			{
				c->token = PPLEX_shift_left;
				p += 2;
			}
			else if (p[1] == '=')
			{
				c->token = PPLEX_less_eq;
				p += 2;
			}
			else
			{
				c->token = *p++;
			}
			break;
		case '>':
			if (p[1] == '>')
			{
				c->token = PPLEX_shift_right;
				p += 2;
			}
			else if (p[1] == '=')
			{
				c->token = PPLEX_greater_eq;
				p += 2;
			}
			else
			{
				c->token = *p++;
			}
			break;
		case '&':
			if (p[1] == '&')
			{
				c->token = PPLEX_and_and;
				p += 2;
			}
			else
			{
				c->token = *p++;
			}
			break;
		case '|':
			if (p[1] == '|')
			{
				c->token = PPLEX_or_or;
				p += 2;
			}
			else
			{
				c->token = *p++;
			}
			break;
		case '=':
			if (p[1] == '=')
			{
				c->token = PPLEX_eq_eq;
				p += 2;
			}
			else
			{
				c->token = *p++;
			}
			break;
		case '!':
			if (p[1] == '=')
			{
				c->token = PPLEX_bang_eq;
				p += 2;
			}
			else
			{
				c->token = *p++;
			}
			break;
		case 0:
			c->token = 0;
			break;
		default:
			c->token = *p++;
			break;

		case 'a':
		case 'b':
		case 'c':
		case 'd':
		case 'e':
		case 'f':
		case 'g':
		case 'h':
		case 'i':
		case 'j':
		case 'k':
		case 'l':
		case 'm':
		case 'n':
		case 'o':
		case 'p':
		case 'q':
		case 'r':
		case 's':
		case 't':
		case 'u':
		case 'v':
		case 'w':
		case 'x':
		case 'y':
		case 'z':
		case 'A':
		case 'B':
		case 'C':
		case 'D':
		case 'E':
		case 'F':
		case 'G':
		case 'H':
		case 'I':
		case 'J':
		case 'K':
		case 'L':
		case 'M':
		case 'N':
		case 'O':
		case 'P':
		case 'Q':
		case 'R':
		case 'S':
		case 'T':
		case 'U':
		case 'V':
		case 'W':
		case 'X':
		case 'Y':
		case 'Z':
		case '_':
		case '$':
		{
			do
				++p;
			while (is_token_character(*p));
			// any Identifier is a macro that WASN'T defined, so it evaluates as 0
			c->token_value = 0;
			c->token = PPLEX_int_literal;
			break;
		}

#define parse_check(c, start, offset, base)						\
	{															\
		char* end;												\
		intmax_t parsed = strtoll(start + offset, &end, base);	\
		if (parsed > INT_MAX || parsed < INT_MIN)				\
		{														\
			c->result = CE_RESULT_overflow;						\
		}														\
		else if (offset && (end == start + offset))				\
		{														\
			c->result = CE_RESULT_syntax_error;					\
		}														\
		else													\
		{														\
			c->token_value = (int)parsed;						\
		}														\
		start = end;											\
	}

		case '0':
			if (is_end_of_number(p[1]))
			{
				c->token_value = 0;
				c->token = PPLEX_int_literal;
				p++;
				break;
			}
			if (p[1] == 'x' || p[1] == 'X')
			{
				parse_check(c, p, 2, 16);
			}
			else if (p[1] == 'b' || p[1] == 'B')
			{
				parse_check(c, p, 2, 2);
			}
			else if (p[1] == '.')
			{
				c->result = CE_RESULT_syntax_error;
			}
			else
			{
				parse_check(c, p, 0, 8);
			}
			goto numeric_suffixes;

		case '1':
		case '2':
		case '3':
		case '4':
		case '5':
		case '6':
		case '7':
		case '8':
		case '9':
			if (is_end_of_number(p[1]))
			{
				c->token_value = *p - '0';
				c->token = PPLEX_int_literal;
				p++;
				break;
			}
			parse_check(c, p, 0, 10);

		numeric_suffixes:
			while (*p == 'u' || *p == 'U' || *p == 'l' || *p == 'L')
			{
				++p;
			}
			c->token = PPLEX_int_literal;
			break;
	}
	c->p = p;
}

static int ce_primary(ppcexp* c)
{
	int v;
	if (c->token != PPLEX_int_literal)
		c->result = CE_RESULT_syntax_error;
	v = c->token_value;
	ce_next(c);
	return v;
}

static int ce_cond(ppcexp* c);
static int ce_unary(ppcexp* c)
{
	int v;
	switch (c->token)
	{
		case '+':
			ce_next(c);
			return ce_unary(c);
		case '-':
			ce_next(c);
			v = ce_unary(c);
			v = -v;
			return v;
		case '!':
			ce_next(c);
			v = ce_unary(c);
			v = (v == 0);
			return v;
		case '~':
			ce_next(c);
			v = ce_unary(c);
			v = ~v;
			return v;
		case '(':
			ce_next(c);
			v = ce_cond(c);
			if (c->token != ')')
				c->result = CE_RESULT_syntax_error;
			ce_next(c);
			return v;
		default:
			return ce_primary(c);
	}
}

static int ce_mul(ppcexp* c)
{
	int left = ce_unary(c);
	while (c->token == '*' || c->token == '/' || c->token == '%')
	{
		int token = c->token;
		int right;
		ce_next(c);
		right = ce_unary(c);
		if (token == '*')
		{
			intmax_t result_wide = (intmax_t)left * (intmax_t)right;
			if (result_wide > INT_MAX || result_wide < INT_MIN)
			{
				c->result = CE_RESULT_overflow;
			}
			left = (int)result_wide;
		}
		else
		{
			if (right == 0)
			{
				if (c->result == CE_RESULT_ok)
				{
					c->result = CE_RESULT_division_by_zero;
				}
			}
			else if (token == '/')
			{
				left /= right;
			}
			else
			{
				left %= right;
			}
		}
	}
	return left;
}

static int ce_sum(ppcexp* c)
{
	int left = ce_mul(c);
	while (c->token == '+' || c->token == '-')
	{
		int token = c->token;
		int right;
		intmax_t result_wide;
		ce_next(c);
		right = ce_mul(c);
		if (token == '+')
		{
			result_wide = (intmax_t)left + (intmax_t)right;
		}
		else
		{
			result_wide = (intmax_t)left - (intmax_t)right;
		}
		if (result_wide > INT_MAX || result_wide < INT_MIN)
		{
			c->result = CE_RESULT_overflow;
		}
		left = (int)result_wide;
	}
	return left;
}

static int ce_shift(ppcexp* c)
{
	int left = ce_sum(c);
	while (c->token == PPLEX_shift_left || c->token == PPLEX_shift_right)
	{
		int tok = c->token;
		int right;
		ce_next(c);
		right = ce_sum(c);
		if (tok == PPLEX_shift_right)
		{
			left = (left >> right);
		}
		else
		{
			left = (left << right);
		}
	}
	return left;
}

static int ce_compare_inequality(ppcexp* c)
{
	int left = ce_shift(c);
	for (;;)
	{
		int right;
		switch (c->token)
		{
			case PPLEX_less_eq:
				ce_next(c);
				right = ce_shift(c);
				left = (left <= right);
				break;
			case PPLEX_greater_eq:
				ce_next(c);
				right = ce_shift(c);
				left = (left >= right);
				break;
			case '<':
				ce_next(c);
				right = ce_shift(c);
				left = (left < right);
				break;
			case '>':
				ce_next(c);
				right = ce_shift(c);
				left = (left > right);
				break;
			default:
				return left;
		}
	}
}

static int ce_compare_equality(ppcexp* c)
{
	int left = ce_compare_inequality(c), right;
	while (c->token == PPLEX_eq_eq || c->token == PPLEX_bang_eq)
	{
		int token = c->token;
		ce_next(c);
		right = ce_compare_inequality(c);
		left = (left == right) ^ (token == PPLEX_bang_eq);
	}
	return left;
}

static int ce_bitwise_and(ppcexp* c)
{
	int left = ce_compare_equality(c), right;
	while (c->token == '&')
	{
		ce_next(c);
		right = ce_compare_equality(c);
		left &= right;
	}
	return left;
}

static int ce_bitwise_xor(ppcexp* c)
{
	int left = ce_bitwise_and(c), right;
	while (c->token == '^')
	{
		ce_next(c);
		right = ce_bitwise_and(c);
		left ^= right;
	}
	return left;
}

static int ce_bitwise_or(ppcexp* c)
{
	int left = ce_bitwise_xor(c), right;
	while (c->token == '|')
	{
		ce_next(c);
		right = ce_bitwise_xor(c);
		left |= right;
	}
	return left;
}

static int ce_logical_and(ppcexp* c)
{
	int left = ce_bitwise_or(c), right;
	while (c->token == PPLEX_and_and)
	{
		int result = c->result;
		ce_next(c);
		right = ce_bitwise_or(c);
		if (left == 0 && c->result == CE_RESULT_division_by_zero && result == CE_RESULT_ok)
			c->result = CE_RESULT_ok;  // suppress unevaluated div by 0
		left = left && right;
	}
	return left;
}

static int ce_logical_or(ppcexp* c)
{
	int left = ce_logical_and(c), right;
	while (c->token == PPLEX_or_or)
	{
		int result = c->result;
		ce_next(c);
		right = ce_logical_and(c);
		if (left != 0 && c->result == CE_RESULT_division_by_zero && result == CE_RESULT_ok)
			c->result = CE_RESULT_ok;  // suppress unevaluated div by 0
		left = left || right;
	}
	return left;
}

static int ce_cond(ppcexp* c)
{
	int cond = ce_logical_or(c);
	if (c->token != '?')
		return cond;
	else
	{
		int v, result, left;
		result = c->result;
		ce_next(c);
		left = ce_cond(c);
		if (c->token != ':')
		{
			c->result = CE_RESULT_syntax_error;
			return cond;
		}
		if (cond == 0 && c->result == CE_RESULT_division_by_zero && result == CE_RESULT_ok)
			c->result = CE_RESULT_ok;  // suppress unevaluated div by 0
		result = c->result;
		ce_next(c);
		v = ce_cond(c);
		if (cond != 0 && c->result == CE_RESULT_division_by_zero && result == CE_RESULT_ok)
			c->result = CE_RESULT_ok;  // suppress unevaluated div by 0
		v = cond != 0 ? left : v;
		return v;
	}
}

int evaluate_integer_constant_expression_as_condition(char* p, int* result)
{
	int v;
	ppcexp c = {0};
	c.p = p;
	ce_next(&c);
	v = ce_cond(&c);
	if (c.token != PPLEX_eof)
		c.result = CE_RESULT_syntax_error;
	*result = c.result;
	return v != 0;
}

int evaluate_integer_constant_expression(char* p, int* result)
{
	int v;
	ppcexp c = {0};
	c.p = p;
	ce_next(&c);
	v = ce_cond(&c);
	if (c.token != PPLEX_eof)
		c.result = CE_RESULT_syntax_error;
	*result = c.result;
	return v;
}
