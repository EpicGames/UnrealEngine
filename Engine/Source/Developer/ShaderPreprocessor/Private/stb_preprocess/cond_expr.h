// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

enum
{
	CE_RESULT_ok,
	CE_RESULT_syntax_error,
	CE_RESULT_division_by_zero,
	CE_RESULT_overflow,
};

#ifdef __cplusplus
#define CONDEXPR_DEF extern "C"
#else
#define CONDEXPR_DEF extern
#endif

CONDEXPR_DEF int evaluate_integer_constant_expression(char* p, int* error);
CONDEXPR_DEF int evaluate_integer_constant_expression_as_condition(char* p, int* error);