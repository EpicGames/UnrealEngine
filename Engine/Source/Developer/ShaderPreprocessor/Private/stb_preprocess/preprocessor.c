// Copyright Epic Games, Inc. All Rights Reserved.

#define _CRT_SECURE_NO_WARNINGS
#include "preprocessor.h"
#include <ctype.h>
#include <limits.h>
#include <stdarg.h>
#include <stdio.h>
#include <stdlib.h>
#include <string.h>
#include "cond_expr.h"
#include "stb_common.h"

#ifdef PPTEST
#define STB_DS_IMPLEMENTATION
#define STB_ALLOC_IMPLEMENTATION
#endif

#include "stb_alloc.h"
#include "stb_ds.h"

#if 0
#include <crtdbg.h>
#define MDBG() _CrtCheckMemory()
#else
#define MDBG()
#endif

// Unreal 5.2+ assumes SSE 4.2 is available on x64 processors
#if (defined(_M_X64) || defined(__amd64__) || defined(__x86_64__)) && !defined(_M_ARM64EC)
#include <immintrin.h>
#define PREPROCESSOR_USE_SSE4_2 1
#define SSE_READ_PADDING 16
#else
#define PREPROCESSOR_USE_SSE4_2 0
#define SSE_READ_PADDING 0
#endif

// Strips whitespace on blank lines and leading whitespace in directives
#define STRIP_BLANK_LINE_WHITESPACE 1

#pragma warning(push)
// arrput calls are incorrectly reported by MSVC analysis to be potentially dereferencing null
// (the maybegrow macro will inline allocate if given a null pointer to begin with)
// not obvious how to get it to sort itself out, so just disabling the warning for expediency
#pragma warning(disable:6011) 

// Silence completely incorrect misparsing of the arrsetlen macros
//-V::521

//////////////////////////////////////////////////////////////////////////////
//
//  Limits to prevent infinite recursion
//

#define MAX_INCLUDE_NESTING 200
#define MAX_MACRO_NESTING 5000
#define MAX_FILENAME 4096

//////////////////////////////////////////////////////////////////////////////
//
//  TYPES
//

typedef unsigned char uint8;
typedef unsigned int uint;
typedef unsigned int uint32;
typedef unsigned short uint16;
typedef signed short int16;
typedef int ppbool;
#ifdef _MSC_VER
typedef unsigned __int64 uint64;
#else
#include <stdint.h>
typedef uint64_t uint64;
#define _vsnprintf vsnprintf
#define _strdup strdup
#endif

// macro expansion helper struct
typedef struct
{
	char* text;
	uint32 text_length;
	int16 argument_index;
	uint8 stringify_argument;
	uint8 eliminate_comma_before_empty_arg;	 // gcc extension using ## to fix up __VA_ARGS__
	uint8 concat_argument;
} expansion_part;

struct macro_definition
{
	char* symbol_name;
	int symbol_name_length;
	union
	{
		char* simple_expansion;
		expansion_part* expansion;	// array
	};
	int simple_expansion_length;
	int num_parameters;	 // if num_parameters < 0, then it doesn't take parentheses when used
	uint8 predefined;
	uint8 is_variadic;	// if true, then __VA_ARGS__ will be specified using 'num_arguments' as argument_index
	uint8 disabled;
	uint8 preprocess_args_first;	// only applies to custom macros
	struct macro_definition* next;	// push/pop stack
};

#define MACRO_NUM_PARAMETERS_no_parentheses -1
#define MACRO_NUM_PARAMETERS_file -2
#define MACRO_NUM_PARAMETERS_line -3
#define MACRO_NUM_PARAMETERS_defined -4
#define MACRO_NUM_PARAMETERS_counter -5
#define MACRO_NUM_PARAMETERS_custom -6

static struct macro_definition predefined_FILE = {"__FILE__", 8, {0}, 0, MACRO_NUM_PARAMETERS_file, 1};
static struct macro_definition predefined_LINE = {"__LINE__", 8, {0}, 0, MACRO_NUM_PARAMETERS_line, 1};
static struct macro_definition predefined_COUNTER = {"__COUNTER__", 11, {0}, 0, MACRO_NUM_PARAMETERS_counter, 1};
static struct macro_definition predefined_defined = {"defined", 7, {0}, 0, MACRO_NUM_PARAMETERS_defined, 1};

#if defined(_MSC_VER) && _MSC_VER <= 1400
#define strdup(x) strdup(x)
#elif !defined(strdup)
#define strdup(x) _strdup(x)
#endif

#if !defined(FORCE_INLINE)
#if defined(_MSC_VER)
#define FORCE_INLINE __forceinline
#else
#define FORCE_INLINE inline __attribute__((always_inline))
#endif
#endif

typedef struct
{
	char* key;
	struct macro_definition* value;
} macro_hash_entry;

// perfect hash table for directives
#define DIRECTIVE_HASH_SIZE 32
struct
{
	char name[8];
	size_t name_len;
} directive_hash[DIRECTIVE_HASH_SIZE];

typedef struct
{
	uint32 line;
	uint32 type;
} ifdef_info;

typedef struct
{
	char* key;	  // filename
	char* value;  // macro name
} include_once;

typedef struct
{
	char* key;
	void* value;
	size_t key_length;
} hashtable_pair;  // 24 bytes

typedef struct
{
	uint32 hash;
	uint32 index;
} table_entry;

typedef struct
{
	char* key;
	void* value;
} shpair;

static loadfile_callback_func loadfile_callback;
static freefile_callback_func freefile_callback;
static resolveinclude_callback_func resolveinclude_callback;
static custommacro_begin_callback_func custommacro_begin;
static custommacro_end_callback_func custommacro_end;

#define HASH_EMPTY_MARKER 0
#define HASH_TOMBSTONE 1

typedef struct
{
	table_entry* table;	   // stb_pp_malloc
	hashtable_pair* pair;  // stb_ds array
	int first_unused_pair;
	uint32 table_size;
	uint32 table_mask;
	uint32 count_in_use;
	uint32 count_tombstones;
	uint32 doubling_threshold;
	struct stb_arena arena;
	shpair* verify;
} pphash;

#define MACRO_BLOOM_FILTER_SIZE 128

typedef struct pp_context
{
	pphash macro_map;				  // map of macro names to their definitions
	macro_hash_entry* undef_map;	  // map of macro names to a flag for whether they've ever been undef'd, for more useful error messages
	include_once* once_map;			  // map of which filenames have been #onced

	// 3KB table by identifier length, with 3 masks indicating whether macros exist with the given key character at a certain location (first,
	// middle, last).  Functions as a trivially cheap bloom filter with 4 keys (length plus the 3 characters), as it has fixed cost regardless
	// of identifier length, and the filter table accesses for a given identifier are cache friendly, since character masks for a given length
	// are adjacent.  Valid identifier characters are in the ASCII range 64-127, so we only need 64 bits of mask.
	//
	// In histogram testing, the middle character was actually the most unique, but all three make a contribution to improving effectiveness.
	// The filter rejects 99.5% of identifiers that aren't macros in HLSL code.
	//
	// The length of the identifier is clamped -- in testing, no identifiers longer than 64 characters were observed.  Using a fixed size
	// table in the structure avoids a couple memory accesses fetching a pointer and current allocated length, which adds up given how
	// often maybe_expand_macro is called.
	uint64 macro_bloom_filter[MACRO_BLOOM_FILTER_SIZE][3];

	int include_nesting_level;	// we stop after a certain number in case of unbounded recursive includes
	int macro_expansion_level;	// because recursive macros are prevented, we don't need to stop this, but we do in case of bugs in recursive macro processing
	char* last_ifndef;
	struct macro_definition* last_macro_definition;
	int include_guard_detect;

	ifdef_info* ifdef_stack;

	pp_where* include_stack;
	pp_diagnostic* diagnostics;
	int stop;  // if we hit an error that causes us to stop processing

	int num_lines;
	int num_disabled_lines;
	int num_onced_files;
	int num_includes;
	int counter;

	struct stb_arena macro_arena;
	void* custom_context;
} pp_context;

typedef struct parse_state
{
	pp_context* context;

	const char* filename; // this is cached in last_output_filename so must remain valid indefinitely
	const char* last_output_filename;
	char* stringified_filename; // this is NULL until first time __FILE__ is used, then it's cached in macro_arena

	const char* src;
	size_t src_offset;
	size_t src_length;

	char* dest;
	size_t copied_identifier_length;			// Set by copy_and_filter_macro, used by maybe_expand_macro

	int src_line_number;
	int dest_line_number;
	int num_disabled_lines;
	int conditional_nesting_depth_at_start;
	int state_limit;

	int include_guard_endif_level;
	char* include_guard_candidate_macro_name;  // this is also the flag for whether we're searching for a matching #endif

	unsigned char in_leading_whitespace;
	unsigned char system_include;

	struct parse_state* parent;
} parse_state;

//////////////////////////////////////////////////////////////////////////////
//
//  UTILITIES
//

static char* arena_alloc_padded_string(struct stb_arena* a, const char* text, size_t stringlen)
{
	// allocate a string with whitespace at each end to prevent accidental token-pasting
	size_t final_length = stringlen + 3;
	char* mem = (char*)stb_arena_alloc_aligned(a, final_length, 4);
	mem[0] = ' ';
	memcpy(mem + 1, text, stringlen);
	mem[stringlen + 1] = ' ';
	mem[stringlen + 2] = 0;
	return mem;
}

static char* arena_alloc_string_trailingspace(struct stb_arena* a, const char* text, size_t stringlen)
{
	// allocate a string with a trailing space appended
	size_t final_length = stringlen + 2;
	char* mem = (char*)stb_arena_alloc_aligned(a, final_length, 4);
	memcpy(mem, text, stringlen);
	mem[stringlen] = ' ';
	mem[stringlen + 1] = 0;
	return mem;

}

#define MIN3(a, b, c) ((a) < (b) ? ((a) < (c) ? (a) : (c)) : ((b) < (c) ? (b) : (c)))

static int compute_edit_distance(const char* s1, const char* s2, int max_dist)
{
	unsigned int s1len, s2len, x, y, lastdiag, olddiag;
	unsigned int column[81] = { 0 };
	s1len = (unsigned int)strlen(s1);
	s2len = (unsigned int)strlen(s2);
	if (s2len < s1len)
	{
		unsigned int ti;
		const char* ts;
		ti = s1len;
		s1len = s2len;
		s2len = ti;
		ts = s1;
		s1 = s2;
		s2 = ts;
	}
	if (s1len + 1 > (unsigned int)sizeof(column))
		return 9999;
	if (s2len > s1len + max_dist)
		return 9999;
	for (y = 1; y <= s1len; y++)
		column[y] = y;
	for (x = 1; x <= s2len; x++)
	{
		column[0] = x;
		for (y = 1, lastdiag = x - 1; y <= s1len; y++)
		{
			olddiag = column[y];
			STB_COMMON_ASSERT(y - 1 < strlen(s1) && y > 1);
			column[y] = MIN3(column[y] + 1, column[y - 1] + 1, lastdiag + (s1[y - 1] == s2[x - 1] ? 0 : 1));
			lastdiag = olddiag;
		}
	}
	return (column[s1len]);
}

static void find_closest_match_init(int* dist, char** best, int max_dist)
{
	*dist = max_dist + 1;
	*best = 0;
}

static void find_closest_match_step(char* reference, char* varying, int* a, char** best)
{
	int dist = compute_edit_distance(reference, varying, *a);
	if (dist < *a)
	{
		*a = dist;
		*best = varying;
	}
}

static char* find_closest_matching_string(char* reference, char* choices[], int num_choices, int max_dist)
{
	int i, dist;
	char* best;

	find_closest_match_init(&dist, &best, max_dist);

	for (i = 0; i < num_choices; ++i)
		find_closest_match_step(reference, choices[i], &dist, &best);

	if (dist <= max_dist)
		return best;
	else
		return NULL;
}

typedef struct
{
	char buf[12];
} err_charbuf12;

static err_charbuf12 error_char(char ch)
{
	err_charbuf12 b;
	if (ch >= 32 && ch < 127)
		if (ch == '\'' || ch == '\\')
			snprintf(b.buf, sizeof(b.buf), "'\\%c'", ch);
		else
			snprintf(b.buf, sizeof(b.buf), "'%c'", ch);
	else if (ch == 0)
		snprintf(b.buf, sizeof(b.buf), "EOF");
	else
		snprintf(b.buf, sizeof(b.buf), "value %d", ch);  // "value -100\0" = 11 (6 + 1 + 3 + 1)
	return b;
}

// write a line number in 12 characters of space, and return a pointer after the number
// because this is used for #line output, we do a few special things:
//     1. 1-digit numbers take 3 characters with two leading spaces
//     2. 2-digit numbers take 2 characters with one leading space
//     3. all other numbers have no extra spaces
//     4. snprintf() is used if number is > 9999, on the assumption this is rare
//            -- but note that if a file is, say, 500K lines, then there may be
//               very few FILES that use this path, but an inordinate number of lines
#define INT_STR_SIZE 12
static char* preprocessor_line_dtoa(char q[INT_STR_SIZE], int number)
{
	static const char digits[10 + 1] = "0123456789";
	static const char digit_pairs[200 + 1] =
		"0001020304050607080910111213141516171819202122232425262728293031323334353637383940414243444546474849"
		"5051525354555657585960616263646566676869707172737475767778798081828384858687888990919293949596979899";

	if (number <= 999)
	{
		static const char first_digit[10 + 1] = " 123456789";
		int n = number % 100;
		int leading = number / 100;
		q[0] = first_digit[leading];
		q[1] = digit_pairs[n * 2 + 0];
		q[2] = digit_pairs[n * 2 + 1];
		if (number < 10)
		{
			q[1] = ' ';
		}
		return q + 3;
	}
	else if (number <= 9999)
	{
		int second = number % 100;
		int first = number / 100;
		q[0] = digit_pairs[first * 2 + 0];
		q[1] = digit_pairs[first * 2 + 1];
		q[2] = digit_pairs[second * 2 + 0];
		q[3] = digit_pairs[second * 2 + 1];
		return q + 4;
	}
	else
	{
		return q + snprintf(q, INT_STR_SIZE, "%d", number);
	}
}

static void output_line_directive(parse_state* cs)
{
	char* q;
	char* out = cs->dest;

	// compute an upper bound on the amount of data to be written
	size_t filename_len = strlen(cs->filename);
	size_t expansion_length =	1 +					// possible newline
								6 +					// "#line "
								INT_STR_SIZE +	// max size of line number string
								2 +					// space + open quote for filename
								filename_len +
								2;					// close quote for filename + newline

	// make sure there's room in the output for the #line directive plus all the remaining text
	arrsetcap(out, arrlen(out) + (size_t)(cs->src_length - cs->src_offset) + expansion_length);

	q = out + arrlen(out);

	// write the line directive
	STB_ASSUME(q != NULL);
	
	// ensure line directives start on a new line
	if (q[-1] != '\n')
		*q++ = '\n';

	*q++ = '#';
	*q++ = 'l';
	*q++ = 'i';
	*q++ = 'n';
	*q++ = 'e';
	*q++ = ' ';
	q = preprocessor_line_dtoa(q, cs->src_line_number);
	*q++ = ' ';
	*q++ = '"';
	strcpy(q, cs->filename);  // @TODO cache stringified filename
	q += filename_len;
	*q++ = '"';
	cs->last_output_filename = cs->filename;
	*q++ = '\n';

	cs->dest_line_number = cs->src_line_number;

	arrsetlen(out, (q - out));
	cs->dest = out;
}

//////////////////////////////////////////////////////////////////////////////
//
//  ERROR HANDLING
//

static uint8 pp_result_mode[PP_RESULT_count] = {
	PP_RESULT_MODE_no_warning, PP_RESULT_MODE_supplementary,
	PP_RESULT_MODE_no_warning,	// #undef of undefined symbol is silently ignored per spec
	PP_RESULT_MODE_warning,
								// initialize all others to PP_RESULT_MODE_stop
};

void pp_set_warning_mode(int result_code, int result_mode)
{
	pp_result_mode[result_code] = (uint8)result_mode;
}

static int did_you_mean_threshold = PP_RESULT_MODE_warning;

void pp_set_did_you_mean_threshold(int mode)
{
	did_you_mean_threshold = mode;
}

void fill_where(parse_state* stack, pp_diagnostic* diagnostic, int* line_number)
{
	pp_where where = { 0 };
	where.filename = STB_COMMON_STRDUP(stack->filename);
	where.line_number = (*line_number > 0 ? *line_number : stack->src_line_number);
	where.column = 0;
	*line_number = 0;
	arrpush(diagnostic->where, where);
}

static int error_explicit(parse_state* ps, int code, int line_number, char* text)
{
	STB_ASSUME(ps != NULL);
	parse_state* stack;
	pp_diagnostic d;

	d.message = text;
	d.diagnostic_code = code;
	d.error_level = pp_result_mode[code];
	if (d.error_level == PP_RESULT_MODE_warning_fast)
		d.error_level = PP_RESULT_MODE_warning;
	d.where = 0;

	fill_where(ps, &d, &line_number);
	stack = ps->parent;
	while (stack != NULL)
	{
		fill_where(stack, &d, &line_number);
		stack = stack->parent;
	}

	arrpush(ps->context->diagnostics, d);

	if (d.error_level == PP_RESULT_MODE_error)
		ps->context->stop = 1;

	return d.error_level == PP_RESULT_MODE_error;
}

static int error_explicit_v(parse_state* ps, int code, int line_number, char* text, va_list va)
{
	size_t len;

#if defined(_MSC_VER) && _MSC_VER < 1500
	len = strlen(text) + 256;
#else
	{
		va_list vc;
#ifdef _MSC_VER
		vc = va;
#else
		va_copy(vc, va);
#endif
		len = _vsnprintf(0, 0, text, vc);
		va_end(vc);
	}
#endif

	size_t size = len + 1;
	// sanity check added to satisfy clang static analyzer
	if (size < 1)
		return 1;

	char* message = (char*)STB_COMMON_MALLOC(size);
	len = _vsnprintf(message, len, text, va);
	
	message[size - 1] = 0;

	return error_explicit(ps, code, line_number, message);
}

// returns TRUE if it should terminate
static int do_error_code(parse_state* ps, int code, char* text, ...)
{
	int err;
	va_list va;

	if (pp_result_mode[code] == PP_RESULT_MODE_no_warning)
	{
		return 0;
	}

	va_start(va, text);
	err = error_explicit_v(ps, code, 0, text, va);
	va_end(va);
	return err;
}

#define do_error_char_code(ps, code, text, c, ...) do_error_code(ps, code, text, error_char(c).buf, ##__VA_ARGS__)

static int do_error(parse_state* ps, char* text, ...)
{
	int err;
	va_list va;

	va_start(va, text);
	err = error_explicit_v(ps, PP_RESULT_ERROR, 0, text, va);
	va_end(va);
	return err;
}

#define do_error_char(ps, text, c, ...) do_error(ps, text, error_char(c).buf, ##__VA_ARGS__)

static int do_error_explicit(parse_state* ps, int code, int line_number, char* text, ...)
{
	int err;
	va_list va;

	if (pp_result_mode[code] == PP_RESULT_MODE_no_warning)
		return 0;

	va_start(va, text);
	err = error_explicit_v(ps, code, line_number, text, va);
	va_end(va);
	return err;
}

static void error_supplement(parse_state* cs, int code, int line_number, char* text, ...)
{
	va_list va;
	if (pp_result_mode[code] == PP_RESULT_MODE_no_warning)
		return;
	va_start(va, text);
	error_explicit_v(cs, PP_RESULT_supplementary, line_number, text, va);
	va_end(va);
}

//////////////////////////////////////////////////////////////////////////////
//
//  STRING HASHING
//
//  based on XXHash32

static unsigned int load_unaligned_32(const void* ptr)
{
#ifdef NO_UNALIGNED_LOADS
	unsigned char* data = ptr;
	return data[0] | (data[1] << 8) | (data[2] << 16) | (data[3] << 24);
#else
	unsigned int* data = (unsigned int*)ptr;  // DOESN'T produce the same result for little-endian & big-endian
	return *data;
#endif
}

#define rotl(x, y) (((x) << y) | ((x) >> (-(y)&31)))
#define xxprime1 0x9E3779B1u
#define xxprime2 0x85EBCA77u
#define xxprime3 0xC2B2AE3Du

// export hash function so compiler can use it as well
unsigned int preprocessor_hash(const char* data, size_t len)
{
	unsigned int hash;
	if (len < 4)
	{
		// load 3 bytes, overlapping if len < 3
		static unsigned char offset1[4] = {0, 0, 1, 1};
		static unsigned char offset2[4] = {0, 0, 0, 2};
		unsigned int h = data[0] + (data[offset1[len]] << 8) + (data[offset2[len]] << 16);
		h = xxprime1 + h * xxprime2;
		hash = rotl(h, 13) * xxprime1;
	}
	else if (len <= 8)
	{
		// load two 32-bit ints, overlapping if len < 8
		unsigned int h0 = xxprime1 + load_unaligned_32(data + 0) * xxprime2;
		unsigned int h1 = xxprime2 + load_unaligned_32(data + len - 4) * xxprime2;
		h0 = rotl(h0, 13) * xxprime1;
		h1 = rotl(h1, 13) * xxprime1;
		hash = rotl(h0, 1) + rotl(h1, 7);
	}
	else
	{
		unsigned int h0 = xxprime1 ^ xxprime2;
		unsigned int h1 = xxprime2;
		unsigned int h2 = 0;
		unsigned int h3 = xxprime1;
		if (len < 16)
		{
			// load four 32-bit ints, partially overlapping
			h0 = h0 + load_unaligned_32(data + 0) * xxprime2;
			h1 = h1 + load_unaligned_32(data + 4) * xxprime2;
			h2 = h2 + load_unaligned_32(data + len - 8) * xxprime2;
			h3 = h3 + load_unaligned_32(data + len - 4) * xxprime2;
			h0 = rotl(h0, 13) * xxprime1;
			h1 = rotl(h1, 13) * xxprime1;
			h2 = rotl(h2, 13) * xxprime1;
			h3 = rotl(h3, 13) * xxprime1;
		}
		else
		{
			while (len > 16)
			{
				h0 = h0 + load_unaligned_32(data + 0) * xxprime2;
				h1 = h1 + load_unaligned_32(data + 4) * xxprime2;
				h2 = h2 + load_unaligned_32(data + 8) * xxprime2;
				h3 = h3 + load_unaligned_32(data + 12) * xxprime2;
				h0 = rotl(h0, 13) * xxprime1;
				h1 = rotl(h1, 13) * xxprime1;
				h2 = rotl(h2, 13) * xxprime1;
				h3 = rotl(h3, 13) * xxprime1;
				data += 16;
				len -= 16;
			}
			data = data + len - 16;
			h0 = h0 + load_unaligned_32(data + 0) * xxprime2;
			h1 = h1 + load_unaligned_32(data + 4) * xxprime2;
			h2 = h2 + load_unaligned_32(data + 8) * xxprime2;
			h3 = h3 + load_unaligned_32(data + 12) * xxprime2;
			h0 = rotl(h0, 13) * xxprime1;
			h1 = rotl(h1, 13) * xxprime1;
			h2 = rotl(h2, 13) * xxprime1;
			h3 = rotl(h3, 13) * xxprime1;
		}
		hash = rotl(h0, 1) + rotl(h1, 7) + rotl(h2, 12) + rotl(h3, 18);
	}
	hash += (unsigned int)len;
	// XXH hash avalanche
	hash ^= hash >> 15;
	hash *= xxprime2;
	hash ^= hash >> 13;
	hash *= xxprime3;
	hash ^= hash >> 16;
	return hash;
}

void stringhash_init_table(pphash* ht, unsigned int size)
{
	table_entry t = {0}, *table;
	unsigned int i;
	STB_ASSUME(size > 0u);
	table = (table_entry*)STB_COMMON_MALLOC(sizeof(table[0]) * size);
	STB_ASSUME(table != NULL);
	ht->table_size = size;
	ht->table_mask = size - 1;
	ht->table = table;
	ht->doubling_threshold = size - (size >> 2);  // 75%
	for (i = 0; i < size; ++i)
		table[i] = t;  // hope for 8-byte-at-a-time init
}

static int fast_preprocess_string_equals(const char* a, const char* b, int length)
{
#if PREPROCESSOR_USE_SSE4_2
	// Does equality compare rather than ordinal compare, and assumes it's safe to read 7 bytes past end of buffer,
	// which is true for strings in the preprocessor, which are in padded arenas, source files, or buffers.
	if (length > 8)
	{
		// Longer string, compare 8 byte words, then a pair of possibly overlapping words at the end.
		for (; length > 16; length-=8, a+=8, b+=8)
		{
			if (*(const uint64*)a != *(const uint64*)b)
			{
				return 0;
			}
		}
		return ((*(const uint64*)a ^ *(const uint64*)b) | (*(const uint64*)(a + length - 8) ^ *(const uint64*)(b + length - 8))) == 0;
	}
	else
	{
		// Short string, compare up to 8 masked bytes.  May read up to 7 bytes past end of input strings.
		return (((uint64)(-1ll)) >> ((8 - length) * 8) & (*(const uint64*)a ^ *(const uint64*)b)) == 0;
	}
#else
	return memcmp(a, b, length) == 0;
#endif
}

// For known short length strings (such as directives)
static int fast_preprocess_string_equals_short(const char* a, const char* b, int length)
{
#if PREPROCESSOR_USE_SSE4_2
	// Short string, compare up to 8 masked bytes.  May read up to 7 bytes past end of input strings.
	return (((uint64)(-1ll)) >> ((8 - length) * 8) & (*(const uint64*)a ^ *(const uint64*)b)) == 0;
#else
	return memcmp(a, b, length) == 0;
#endif
}

void* stringhash_get(pphash* ht, const char* key, size_t keylen)
{
	uint32 mask = ht->table_mask;
	uint32 hash = preprocessor_hash(key, keylen);
	uint32 slot = hash & mask;
	uint32 step;
	void* vresult = 0;
	table_entry* table = ht->table;

	if (ht->verify)
	{
		char temp[1024];
		memcpy(temp, key, keylen);
		temp[keylen] = 0;
		vresult = shget(ht->verify, temp);
	}

	if (hash <= HASH_TOMBSTONE)
		hash += 2;
	if (table[slot].hash == hash)
	{
		uint32 i = table[slot].index;
		if (ht->pair[i].key_length == keylen && fast_preprocess_string_equals(key, ht->pair[i].key, keylen))
		{
			if (ht->verify)
				STB_COMMON_ASSERT(vresult == ht->pair[i].value);
			return ht->pair[i].value;
		}
	}
	if (table[slot].hash == HASH_EMPTY_MARKER)
	{
		STB_COMMON_ASSERT(vresult == NULL);
		return NULL;
	}

	step = hash ^ (hash >> 16) ^ (hash >> 23);
	step |= 1;
	for (;;)
	{
		slot = (slot + step) & mask;
		if (table[slot].hash == hash)
		{
			uint32 i = table[slot].index;
			if (ht->pair[i].key_length == keylen && fast_preprocess_string_equals(key, ht->pair[i].key, keylen))
			{
				if (ht->verify)
					STB_COMMON_ASSERT(vresult == ht->pair[i].value);
				return ht->pair[i].value;
			}
		}
		if (table[slot].hash == HASH_EMPTY_MARKER)
		{
			STB_COMMON_ASSERT(vresult == NULL);
			return NULL;
		}
	}
}

int stringhash_delete(pphash* ht, char* key, size_t keylen)
{
	uint32 mask = ht->table_mask;
	uint32 hash = preprocessor_hash(key, keylen);
	uint32 slot = hash & mask;
	uint32 step, i;
	table_entry* table = ht->table;

	if (ht->verify)
	{
		char temp[1024];
		memcpy(temp, key, keylen);
		temp[keylen] = 0;
		shdel(ht->verify, temp);
	}

	if (hash <= HASH_TOMBSTONE)
		hash += 2;
	if (table[slot].hash == hash)
	{
		i = table[slot].index;
		if (ht->pair[i].key_length == keylen && fast_preprocess_string_equals(key, ht->pair[i].key, keylen))
			goto del;
	}
	if (table[slot].hash == HASH_EMPTY_MARKER)
		return 0;

	step = hash ^ (hash >> 16) ^ (hash >> 23);
	step |= 1;
	for (;;)
	{
		slot = (slot + step) & mask;
		if (table[slot].hash == hash)
		{
			i = table[slot].index;
			if (ht->pair[i].key_length == keylen && fast_preprocess_string_equals(key, ht->pair[i].key, keylen))
				goto del;
		}
		if (table[slot].hash == HASH_EMPTY_MARKER)
			return 0;
	}
	del : i = table[slot].index;
	ht->pair[i].key = 0;
	ht->pair[i].value = 0;
	ht->pair[i].key_length = (size_t)ht->first_unused_pair;
	ht->first_unused_pair = (int)i;
	table[slot].hash = HASH_TOMBSTONE;
	return 1;
}

static void stringhash_insert_ref(pphash* ht, table_entry e)
{
	uint32 mask = ht->table_mask;
	uint32 hash = e.hash;
	uint32 slot = hash & mask;
	uint32 step;
	table_entry* table = ht->table;
	if (hash <= HASH_TOMBSTONE)
		hash += 2;

	if (table[slot].hash != HASH_EMPTY_MARKER)
	{
		step = hash ^ (hash >> 16) ^ (hash >> 23);
		step |= 1;
		for (;;)
		{
			slot = (slot + step) & mask;
			if (table[slot].hash == HASH_EMPTY_MARKER)
				break;
		}
	}
	table[slot] = e;
}

void stringhash_put(pphash* ht, const char* key, size_t keylen, void* value)
{
	size_t n;
	hashtable_pair p;
	uint32 mask = ht->table_mask;
	uint32 hash = preprocessor_hash(key, keylen);
	uint32 slot = hash & mask;
	uint32 step;
	table_entry* table = ht->table;
	if (hash <= HASH_TOMBSTONE)
		hash += 2;

	if (ht->verify)
	{
		STB_COMMON_ASSERT(stringhash_get(ht, key, keylen) == NULL);
	}
	if (ht->verify)
	{
		char temp[1024];
		memcpy(temp, key, keylen);
		temp[keylen] = 0;
		shput(ht->verify, temp, value);
	}

	if (table[slot].hash > HASH_TOMBSTONE)
	{
		step = hash ^ (hash >> 16) ^ (hash >> 23);
		step |= 1;
		for (;;)
		{
			slot = (slot + step) & mask;
			if (table[slot].hash <= HASH_TOMBSTONE)
				break;
		}
	}
	if (table[slot].hash == HASH_TOMBSTONE)
		--ht->count_tombstones;

	p.key = (char*)stb_arena_alloc_aligned(&ht->arena, keylen + 1, 1);
	p.key_length = keylen;
	p.value = value;
	memcpy(p.key, key, keylen);
	p.key[keylen] = 0;
	if (ht->first_unused_pair >= 0)
	{
		// allocate existing slot from ht->pair
		n = ht->first_unused_pair;
		ht->first_unused_pair = (int)ht->pair[n].key_length;
		ht->pair[n] = p;
	}
	else
	{
		// allocate new slot in ht->pair
		arrput(ht->pair, p);
		n = arrlen(ht->pair) - 1;
	}

	table[slot].hash = hash;
	table[slot].index = (uint32)n;
	++ht->count_in_use;

	if (ht->count_in_use + ht->count_tombstones >= ht->doubling_threshold)
	{
		table = ht->table;
		uint32 size = ht->table_size;
		uint32 i;
		stringhash_init_table(ht, size * 2);
		ht->count_tombstones = 0;
		for (i = 0; i < size; ++i)
			if (table[i].hash > HASH_TOMBSTONE)
				stringhash_insert_ref(ht, table[i]);
		STB_COMMON_FREE(table);
	}
}

void stringhash_create(pphash* ht, int init_size)  // must be power of two or 0
{
	memset(ht, 0, sizeof(pphash));
	ht->first_unused_pair = -1;
	stringhash_init_table(ht, init_size < 4 ? 16 : init_size);
	ht->verify = 0;
	// sh_new_arena(ht->verify);
}

void stringhash_destroy(pphash* ht)
{
	stb_arena_free(&ht->arena);
	arrfree(ht->pair);
	STB_COMMON_FREE(ht->table);
	if (ht->verify)
		shfree(ht->verify);
}

#ifdef HASHTEST

char* get_string(int n)
{
	unsigned int bits = (unsigned int)n;
	static char buffer[40];
	int bottom_bits = (n & 3);
	int rest = (n >> 2);
	int len = ((rest * 21) & 31), i;
	if (bottom_bits < 2)
		len = (len >> 2) + 3;
	else if (bottom_bits == 2)
		len = (len >> 1) + 3;
	else
		len = len + 3;
	if ((n >> 18) > 0)
		++len;
	len += 5;

	bits *= xxprime3;
	bits ^= bits >> 15;
	bits = rotl(bits, 19);
	for (i = 0; i < len; ++i)
	{
		buffer[i] = (bits & 63) + 33;
		bits = rotl(bits, 7);
	}
	// guarantee that first 2^24 strings are unique
	buffer[0] = (n & 63) + 33;
	buffer[1] = ((n >> 6) & 63) + 33;
	buffer[2] = ((n >> 12) & 63) + 33;
	buffer[3] = ((n >> 18) & 63) + 33;
	buffer[len] = 0;
	return buffer;
}

void test_stringhash(void)
{
	int i, n;
	pphash h;
	for (n = 6; n < 5000000; n *= 3)
	{
		stringhash_create(&h, 16);
		for (i = 1; i < n; ++i)
		{
			char* z = get_string(i);
			stringhash_put(&h, z, strlen(z), (void*)(size_t)(i));
		}
		stringhash_destroy(&h);

		stringhash_create(&h, 16);
		for (i = 1; i < n; ++i)
		{
			char* z = get_string(i);
			stringhash_put(&h, z, strlen(z), (void*)(size_t)(i));
		}
		for (i = 1; i < n; ++i)
		{
			char* z = get_string(i);
			void* v = stringhash_get(&h, z, strlen(z));
			STB_COMMON_ASSERT(v == (void*)(size_t)i);
		}
		for (i = 1; i < n; i += 2)
		{
			char* z = get_string(i);
			int r = stringhash_delete(&h, z, strlen(z));
			STB_COMMON_ASSERT(r == 1);
		}
		for (i = 1; i < n; ++i)
		{
			char* z = get_string(i);
			void* v = stringhash_get(&h, z, strlen(z));
			STB_COMMON_ASSERT(v == ((i & 1) ? 0 : (void*)(size_t)i));
		}
		for (i = 1; i < n; ++i)
		{
			char* z = get_string(i);
			int r = stringhash_delete(&h, z, strlen(z));
			STB_COMMON_ASSERT(r == ((i & 1) ? 0 : 1));
		}
		stringhash_destroy(&h);
	}
}
#endif

//////////////////////////////////////////////////////////////////////////////
//
//  SCANNERS
//
//    The following routines scan over characters looking for something.
//
//    It is expected that most per-character time will be spent in the
//    scanners, thus we want high performance for many of them.
//
//    We have three classes of scanners, in order of speed:
//
//    1.  pp_char_flags scanners
//    2.  state machine scanners
//    3.  C-style scanners
//
//  pp_char_flags scanners
//
//    There are multiple character classifications which are assigned to
//    individual bits of the 8-bit-per-character pp_char_flags[] array.
//    A scanner can look for multiple bits at once, allowing this to be
//    used for more than 8 kinds of scanning.
//
//  State Machine scanners
//
//    These scanners have a complex state transition diagram encoded
//    in data tables which allows them to parse things like whitespace
//    and comments simultaneously in a single fast loop.
//
//  C-style scanners
//
//    A C-style scanner checks explicitly for all of the cases it cares
//    about. It can do special multi-character processing, but is the
//    slowest kind of scanner.

//
//  pp_char_flags scanners
//

static unsigned char pp_char_flags1[256];
static unsigned char pp_char_flags2[256];

#define CHAR1_IS_NUL 1
#define CHAR1_IS_NONNEWLINE_WHITESPACE 2
#define CHAR1_IS_NEWLINE 4
#define CHAR1_IS_BALANCE_SCAN_ACTION 8	//  ( , ) EOF
#define CHAR1_IS_PP_IDENTIFIER_FIRST 16
#define CHAR1_IS_DIGIT 32
#define CHAR1_IS_QUOTE 64
#define CHAR1_IS_BACKSLASH 128

#define CHAR2_IS_NUL 1
#define CHAR2_IS_SLASH 2
#define CHAR2_IS_NEWLINE 4

#define char_is_pp_identifier_first(x) (pp_char_flags1[(uint8)(x)] & CHAR1_IS_PP_IDENTIFIER_FIRST)
#define char_is_pp_identifier(x) (pp_char_flags1[(uint8)(x)] & (CHAR1_IS_PP_IDENTIFIER_FIRST | CHAR1_IS_DIGIT))
#define char_is_nonnewline_whitespace(x) (pp_char_flags1[(uint8)(x)] & CHAR1_IS_NONNEWLINE_WHITESPACE)
#define char_is_newline(x) (pp_char_flags1[(uint8)(x)] & CHAR1_IS_NEWLINE)
#define char_is_whitespace(x) (pp_char_flags1[(uint8)(x)] & (CHAR1_IS_NONNEWLINE_WHITESPACE | CHAR1_IS_NEWLINE))
#define char_is_nul(x) ((x) == 0)
#define char_is_balance_scan_action(x) (pp_char_flags1[(uint8)(x)] & (CHAR1_IS_BALANCE_SCAN_ACTION | CHAR1_IS_NUL))
#define char_is_end_of_line(x) (pp_char_flags1[(uint8)(x)] & (CHAR1_IS_NEWLINE | CHAR1_IS_NUL))
#define char_is_whitespace_or_eof(x) (pp_char_flags1[(uint8)(x)] & (CHAR1_IS_NONNEWLINE_WHITESPACE | CHAR1_IS_NEWLINE | CHAR1_IS_NUL))
#define char_is_special_string_contents(x) (pp_char_flags1[(uint8)(x)] & (CHAR1_IS_QUOTE | CHAR1_IS_BACKSLASH | CHAR1_IS_NUL))
#define char_is_quote(x) (pp_char_flags1[(uint8)(x)] & CHAR1_IS_QUOTE)
#define char_is_argument_parse_action(x) \
	(pp_char_flags1[(uint8)(x)] & (CHAR1_IS_BALANCE_SCAN_ACTION | CHAR1_IS_QUOTE | CHAR1_IS_BACKSLASH | CHAR1_IS_NUL | CHAR1_IS_NEWLINE))
#define char_is_copy_line_exception(x) (pp_char_flags2[(uint8)(x)] & (CHAR2_IS_SLASH | CHAR2_IS_NEWLINE | CHAR2_IS_NUL))

#ifdef TREAT_LF_CR_AS_TWO_NEWLINES
// returns true if CR LF
#define is_twochar_newline_given_first_is_newline(a, b) ((a) == '\r' && (b) == '\n')
//#define is_twochar_newline_given_first_is_newline(a,b)  ((a) == '\r' & (b) == '\n') // potentially faster but probably slower
#else
// returns true if CR LF or LF CR
#define is_twochar_newline_given_first_is_newline(a, b) ((a) + (b) == '\r' + '\n')
#endif

#define newline_char_count(newline_char, following_char) (is_twochar_newline_given_first_is_newline(newline_char, following_char) ? 2 : 1)

#define is_backslash_newline(p) ((p)[0] == '\\' && char_is_newline((p)[1]))

// this is used when parsing a backslash-eliminated directive
static const char* preprocessor_skip_whitespace_simple(const char* s)
{
	while (char_is_nonnewline_whitespace(*s))
		++s;
	return s;
}

// This is used when trimming macro arguments of leading whitespace
static char* preprocessor_skip_whitespace_fast(char* s)
{
	// same as preprocessor_skip_whitespace, but doesn't need to count newlines
	for (;;)
	{
		while (char_is_whitespace(*s))
			++s;

		// handle backslash continuation
		if (s[0] == '\\' && char_is_newline(s[1]))
			s += 2;
		else
			return s;
	}
}

// This is used when trimming macro arguments of trailing whitespace
// @OPTIMIZE: turn this into a smart state-machine scanner?
static const char* preprocessor_skip_whitespace_reverse(const char* s)
{
	for (;;)
	{
		while (char_is_whitespace(s[-1]))
			--s;

		// handle backslash continuation
		if (s[-1] == '\\' && char_is_newline(s[1]))
		{
			--s;
		}
		else
			return s;
	}
}

static const char* scan_pp_identifier(const char* p)
{
	STB_COMMON_ASSERT(char_is_pp_identifier_first(*p));
	++p;
	while (char_is_pp_identifier(*p))
		++p;
	return p;
}

// after running fast whitespace & comment scanner, we lose sync with start-of-line,
// which will break column numbers in compiler, so this helper function recovers
static void rewind_to_start_of_line(parse_state* cs)
{
	const char* q = cs->src;
	const char* p = cs->src + cs->src_offset;

	// stop at end of line or non-whitespace (which actually should only be able to be end of a multiline comment, i.e. '/')
	while (p > q && char_is_nonnewline_whitespace(p[-1]))
		--p;

	cs->src_offset = p - q;
	cs->in_leading_whitespace = 1;
}

//
//  State Machine scanners
//

enum
{
	PP_CHAR_CLASS_default,
	PP_CHAR_CLASS_whitespace,
	PP_CHAR_CLASS_cr,
	PP_CHAR_CLASS_lf,
	PP_CHAR_CLASS_period,
	PP_CHAR_CLASS_digit,
	PP_CHAR_CLASS_idtype,  // letter, _, or $
	PP_CHAR_CLASS_idtype_e_E,
	PP_CHAR_CLASS_apostrophe,
	PP_CHAR_CLASS_quotation,

	PP_CHAR_CLASS_slash,
	PP_CHAR_CLASS_backslash,
	PP_CHAR_CLASS_star,
	PP_CHAR_CLASS_hash,
	PP_CHAR_CLASS_plusminus,

	PP_CHAR_CLASS_eof,

	PP_CHAR_CLASS_count,
};

enum
{
	PP_STATE_ready,
	PP_STATE_in_leading_whitespace,
	PP_STATE_saw_slash,
	PP_STATE_comment_singleline,
	PP_STATE_comment_multiline,
	PP_STATE_comment_multiline_seen_star,
	PP_STATE_comment_multiline_seen_cr,
	PP_STATE_comment_multiline_seen_lf,
	PP_STATE_in_string_literal,
	PP_STATE_in_string_literal_saw_backslash,
	PP_STATE_in_char_literal,
	PP_STATE_in_char_literal_saw_backslash,
	PP_STATE_saw_period,
	PP_STATE_in_number,
	PP_STATE_in_number_saw_e,
	PP_STATE_saw_cr,
	PP_STATE_saw_lf,

	PP_STATE_active_count,

	PP_STATE_saw_backslash,

	PP_STATE_eof,
	PP_STATE_saw_leading_hash,
	PP_STATE_saw_hash,
	PP_STATE_saw_identifier_start,

	PP_STATE_error_unterminated_multiline_comment,

	PP_STATE_count
};

// here are possible exit states, and their meanings:
//
//  PP_STATE_saw_cr,                               // if we need to emit #line
//  PP_STATE_saw_lf,                               // if we need to emit #line
//  PP_STATE_saw_backslash,                        // backslash at end of line or syntax error, but not in preprocessor directive such as macro definition
//  PP_STATE_eof,                                  // end of file
//  PP_STATE_saw_leading_hash,                     // preprocessor directive
//  PP_STATE_saw_hash,                             // syntax error    // @TODO: get rid of this case and let the compiler detect and report the error
//  PP_STATE_saw_identifier_start,                 // any identifier
//  PP_STATE_error_unterminated_multiline_comment  // syntax error

enum
{
	PP_WHITESCAN_STATE_whitespace,
	PP_WHITESCAN_STATE_saw_cr,
	PP_WHITESCAN_STATE_saw_lf,
	PP_WHITESCAN_STATE_saw_slash,
	PP_WHITESCAN_STATE_comment_singleline,
	PP_WHITESCAN_STATE_comment_multiline,
	PP_WHITESCAN_STATE_comment_multiline_seen_cr,
	PP_WHITESCAN_STATE_comment_multiline_seen_lf,
	PP_WHITESCAN_STATE_comment_multiline_seen_star,

	PP_WHITESCAN_STATE_active_count,
	PP_WHITESCAN_STATE_stop_seen_onechar,
	PP_WHITESCAN_STATE_stop_seen_twochars,
	PP_WHITESCAN_STATE_stop_eof,
	PP_WHITESCAN_STATE_error_unterminated_multiline_comment,
};

static unsigned char pp_char_class[256];
static unsigned char pp_scan_char_class[256];							 // same as above, but identifiers are treated as default
static unsigned char pp_transition_table[PP_STATE_active_count][16];	 // normal transition table
static unsigned char pp_transition_initscan[PP_STATE_active_count][16];	 // special transition table while scanning for initial #ifndef
static unsigned char pp_state_is_end_of_line[PP_STATE_count];
static unsigned char pp_initscan_state_is_end_of_line[PP_STATE_count];

static void output_line_directive(parse_state* cs);

// To implement fast include-guard skipping, we want to detect #ifndef at start of file.
// So we start by scanning whitespace and comments until we hit the first non-whitespace
// characters in the file. We don't handle backslash concatenation, so no fast include-guard if so.
static int scan_whitespace_and_comments(parse_state* cs)
{
	const char* p = cs->src + cs->src_offset;
	int state = PP_WHITESCAN_STATE_whitespace;
	int line_count = cs->src_line_number;

	do
	{
		uint8 ch = (uint8)*p++;
		uint8 ch_class = pp_char_class[ch];
		state = pp_transition_initscan[state][ch_class];
		line_count += pp_initscan_state_is_end_of_line[state];
	} while (state < PP_WHITESCAN_STATE_active_count);

	cs->src_line_number = line_count;

	if (state == PP_WHITESCAN_STATE_stop_seen_twochars)
	{
		// we've processed two characters, and incremented past both of them
		p -= 2;
	}
	else
	{
		p -= 1;
	}

	cs->src_offset = p - cs->src;

	return state;
}

// Returns NULL if possibly a macro, or the end of the identifier if definitely not a macro
static FORCE_INLINE const char* copy_and_filter_macro(parse_state* cs, const char* in, char* out)
{
	pp_context* c = cs->context;
	const char* p = in;
	char* q = out;
	const char* start = p;
	char ch_first = *p;

#if PREPROCESSOR_USE_SSE4_2
	__m128i k_identifier_needle = _mm_setr_epi8('A', 'Z', 'a', 'z', '0', '9', '_', '_', '$', '$', '$', '$', '$', '$', '$', '$');

	__m128i identifier_word = _mm_loadu_si128((const __m128i*)p);
	int end_index = _mm_cmpistri(k_identifier_needle, identifier_word, _SIDD_UBYTE_OPS | _SIDD_CMP_RANGES | _SIDD_NEGATIVE_POLARITY);

	_mm_storeu_si128((__m128i*)q, identifier_word);
	p += end_index;
	q += end_index;

	// Write as long we had a full word
	while (end_index == 16)
	{
		identifier_word = _mm_loadu_si128((const __m128i*)p);
		end_index = _mm_cmpistri(k_identifier_needle, identifier_word, _SIDD_UBYTE_OPS | _SIDD_CMP_RANGES | _SIDD_NEGATIVE_POLARITY);
		if (!end_index)
		{
			break;
		}
		_mm_storeu_si128((__m128i*)q, identifier_word);
		p += end_index;
		q += end_index;
	}

#else  // PREPROCESSOR_USE_SSE4_2

	// copy the rest of the identifier so we can NUL-terminate it
	// we copy it to the output buffer since we've already allocated space for it,
	// and that's where we'll need to copy it if it's not a macro
	// There is always at least 1 identifier character, so copy the first unconditionally
	p++;
	*q++ = ch_first;
	while (char_is_pp_identifier(*p))
		*q++ = *p++;

#endif  // !PREPROCESSOR_USE_SSE4_2

	size_t identifier_length = p - start;

	// Test bits for three key characters of the macro by length (first, middle, last)
	size_t bloom_filter_offset = identifier_length < MACRO_BLOOM_FILTER_SIZE ? identifier_length : MACRO_BLOOM_FILTER_SIZE - 1;
	if (!(c->macro_bloom_filter[bloom_filter_offset][0] & (1ull << (ch_first & 0x3f))) ||
		!(c->macro_bloom_filter[bloom_filter_offset][1] & (1ull << (start[identifier_length >> 1] & 0x3f))) ||
		!(c->macro_bloom_filter[bloom_filter_offset][2] & (1ull << (start[identifier_length - 1] & 0x3f))))
	{
		cs->src_offset = p - cs->src;
		arrsetlennocap(cs->dest, (size_t)(q - cs->dest));  // set the output buffer size to account for the above-copied identifier
		return p;		// Not a macro, return end of parse
	}

	// There is an implicit contract that "maybe_expand_macro" should consume identifiers copied by "copy_and_filter_macro".
	// To allow validation of this, the length starts as zero, is set to non-zero here, then back to zero in "maybe_expand_macro".
	STB_COMMON_ASSERT(cs->copied_identifier_length == 0);

	cs->copied_identifier_length = identifier_length;
	return NULL;		// Could be a macro, return NULL
}

// Following routine is the workhorse; this copies from a file's text buffer
// into the output buffer until it finds a directive, a preprocessor Identifier
// (which might be a macro name), or the end of the file.
//
// It also performs backslash-continuation elimination.

static int copy_to_action_point(parse_state* cs)
{
	int state;
	STB_COMMON_ASSERTF(cs->src != NULL && cs->src_offset <= cs->src_length,
		"source buffer null or offset past end (filename=%s, src=%p, src_length=%zu, src_offset=%zu)",
		cs->filename ? cs->filename : "<null>", (const void*)cs->src, cs->src_length, cs->src_offset);
	STB_ASSUME(cs->src != NULL);
	const char* p = cs->src + cs->src_offset;
	char* out;
	char* q = NULL;

	out = cs->dest;

	state = cs->in_leading_whitespace ? PP_STATE_in_leading_whitespace : PP_STATE_ready;
	MDBG();

	for (;;)
	{
		int prev_state;

		// expand output so there's room to copy entire file unmodified, plus padding so SSE operations can write to the end of the buffer without special cases.
		// +1 accounts for the NUL terminator byte that the inner loop writes before the exit condition is checked.
		int out_len = arrlen(out);
		arrsetcap(out, out_len + (cs->src_length - cs->src_offset) + SSE_READ_PADDING + 1);
		cs->dest = out;  // arrsetcap may realloc 'out' to a new address; keep cs->dest in sync so copy_and_filter_macro uses a valid pointer
		q = out + out_len;

#if STRIP_BLANK_LINE_WHITESPACE
		char* q_linestart = q;
#endif

		{  // make lifetime of line_count unambiguous
			const int state_limit = cs->state_limit;
			int line_count = 0;

		resume_parsing:
			do
			{
#if STRIP_BLANK_LINE_WHITESPACE
				uint8 ch = (uint8)*p++;
				uint8 ch_class;
				STB_COMMON_ASSERT(q < out + arrcap(out));
				STB_ASSUME(q != NULL);
				ch_class = pp_char_class[ch];
				prev_state = state;
				STB_ASSUME(state < PP_STATE_active_count && ch_class < PP_CHAR_CLASS_count);
				state = pp_transition_table[state][ch_class];
				if (ch == '\n')				// pp_state_is_end_of_line[state] -- Assume newlines are normalized
				{
					// Eliminate leading whitespace before a newline, by backing up to the line start
					if (prev_state == PP_STATE_in_leading_whitespace)
					{
						q = q_linestart;
					}
					line_count++;
					q_linestart = q + 1;
				}
				*q++ = ch;
#else
				uint8 ch = (uint8)*p++;
				uint8 ch_class;
				STB_COMMON_ASSERT(q < out + arrcap(out));
				STB_ASSUME(q != NULL);
				*q++ = ch;
				ch_class = pp_char_class[ch];
				prev_state = state;
				STB_ASSUME(state < PP_STATE_active_count && ch_class < PP_CHAR_CLASS_count);
				state = pp_transition_table[state][ch_class];
				line_count += (ch == '\n') ? 1 : 0;				// pp_state_is_end_of_line[state] -- Assume newlines are normalized, avoid memory read
#endif
			} while (state < state_limit);

			if (state == PP_STATE_saw_identifier_start)
			{
				// Copy the identifier and check if it might be a macro.  The first identifier character will already have been written
				// by the parse loop, so we need to go back one.
				const char* identifier_start = p - 1;
				const char* identifier_end = copy_and_filter_macro(cs, identifier_start, q - 1);

				if (identifier_end)
				{
					// If not a macro, we can resume parsing
					q += identifier_end - p;
					p = identifier_end;
					state = PP_STATE_ready;
					cs->in_leading_whitespace = 0;
					goto resume_parsing;
				}
				else
				{
					// Breaking out of the loop skips the line increment below, so do it here as well
					cs->src_line_number += line_count;
					cs->dest_line_number += line_count;
					break;
				}
			}

			cs->src_line_number += line_count;
			cs->dest_line_number += line_count;
		}

		// The cases that are not syntax errors and not once per file:
		//    need to emit #line
		//    preprocessor directive
		//    any identifier
		//    backlash at end of non-preprocessor directive
		//
		// of those, presumably most common case is going to be an identifier
		// (which we have to check to see if it's a macro). So we should handle
		// that case first. We can use the filters that we use to detect identifiers
		// that aren't macros to ALSO detect not any of the other cases, i.e. naively if
		// the filter returns true it's because it MUST be something other than
		// an identifier that's not a macro, but we can also make it ONLY return true
		// if the input was an identifier and apply it BEFORE checking we're in the
		// identifier case

		if (state > PP_STATE_saw_backslash)
		{
#if STRIP_BLANK_LINE_WHITESPACE
			if (state == PP_STATE_saw_leading_hash)
			{
				// Eliminate leading whitespace before a hash, by backing up to the line start.  Need to add one
				// as we subtract one below to remove what would have been the last character copied (the hash),
				// if we weren't backing up to line start.
				q = q_linestart + 1;
			}
#endif
			break;
		}

		if (state == PP_STATE_saw_backslash)
		{
			STB_COMMON_ASSERT(p[-1] == '\\');
			if (!char_is_newline(*p))
			{
				// if there's not a newline, just treat backslash as normal character and continue
				state = PP_STATE_ready;
			}
			else
			{
				// join the lines together
				--q;								  // undo copying the backlash
				p += newline_char_count(p[0], p[1]);  // skip past CR, LF, CR LF, LF CR
				cs->src_line_number += 1;			  // we've seen it in the source but haven't seen it in the dest
				cs->state_limit = PP_STATE_saw_cr;	// line_number mismatches, so need to output match newlines at first opportunity, namely next time we see a
													// newline not in a comment
				state = prev_state;					// reset parse state to before the backslash
				arrsetlen(out, (size_t)(q - out));
			}
		}
		else
		{
			// after the above backslash code runs, this code
			// is run on the next CR or LF

			STB_COMMON_ASSERT(p[-1] == '\n' || p[-1] == '\r');

			// finish copying a CR LF or LF CR pair
			if (is_twochar_newline_given_first_is_newline(p[-1], p[0]))
				*q++ = *p++;

			// reset state
			cs->state_limit = PP_STATE_active_count;

			// output newlines to the output buffer until synchronized again
			arrsetlen(out, (size_t)(q - out));
			while (cs->dest_line_number < cs->src_line_number)
			{
				arrput(out, '\n');
				++cs->dest_line_number;
			}
		}
	}

	// set output to the amount we actually copied... don't count the copy of the last character
	arrsetlennocap(out, (size_t)(q - 1 - out));
	cs->dest = out;

	cs->src_offset = (size_t)(p - 1 - cs->src);
	cs->in_leading_whitespace = (state == PP_STATE_saw_leading_hash);

	return state;
}

// after performing macro expansion, we rescan the output for further
// macros. this doesn't need to do newline processing or handle directives.
// we use the same states as above and just should never hit the other ones.
// because we're not doing backslash-continuations, the inner loop can
// be simpler (no tracking of previous state)...
//
// (we do need to parse and // skip newlines, since they can be copied from
// macro arguments, but we don't need to COUNT them)
static int copy_to_action_point_macro_expansion(parse_state* cs)
{
	int state;
	STB_COMMON_ASSERTF(cs->src != NULL && cs->src_offset <= cs->src_length,
		"source buffer null or offset past end during macro expansion (filename=%s, src=%p, src_length=%zu, src_offset=%zu)",
		cs->filename ? cs->filename : "<null>", (const void*)cs->src, cs->src_length, cs->src_offset);
	STB_ASSUME(cs->src != NULL);
	const char* p = cs->src + cs->src_offset;
	char* out = cs->dest;
	STB_ASSUME(out != NULL);
	char* q;

	// expand output so copying entire file unmodified is possible.
	// +1 accounts for the NUL terminator byte that the inner loop writes before the exit condition is checked.
	int out_len = arrlen(out);
	arrsetcap(out, out_len + (cs->src_length - cs->src_offset) + SSE_READ_PADDING + 1);
	cs->dest = out;  // arrsetcap may realloc 'out' to a new address; keep cs->dest in sync so copy_and_filter_macro uses a valid pointer
	q = out + out_len;

	STB_ASSUME(q != NULL);

	state = PP_STATE_ready;

	for (;;)
	{
		do
		{
			uint8 ch = (uint8)*p++;
			uint8 ch_class;
			*q++ = ch;
			ch_class = pp_char_class[ch];
			state = pp_transition_table[state][ch_class];
		} while (state < PP_STATE_active_count);

		if (state == PP_STATE_saw_identifier_start)
		{
			// Copy the identifier and check if it might be a macro.  The first identifier character will already have been written
			// by the parse loop, so we need to go back one.
			const char* identifier_start = p - 1;
			const char* identifier_end = copy_and_filter_macro(cs, identifier_start, q - 1);

			if (identifier_end)
			{
				q += identifier_end - p;
				p = identifier_end;
				state = PP_STATE_ready;
				cs->in_leading_whitespace = 0;
			}
			else
			{
				break;
			}
		}
		else
		{
			break;
		}
	}

	cs->src_offset = (size_t)(p - 1 - cs->src);

	// set output to the amount we actually copied... don't count the copy of the last character
	arrsetlennocap(out, (size_t)(q - 1 - out));
	cs->dest = out;

	return state;
}

// when in a conditionally-disabled section of code, we want to
// scan as fast as possible to the next directive; need a state-machine
// to handle comments efficiently
static const char* scan_to_directive(const char* p, int* p_line_number)
{
	int line_number = *p_line_number;

#if PREPROCESSOR_USE_SSE4_2
	// Fast loop assumes comments have been stripped, and newlines normalized.  Searches for a # not in quoted text.  Range is the
	// negation of the characters we want, which will also stop at the null terminator.  Around 10x faster than original loop.
	// Note that the source may still contain comments added via DumpShaderDefinesAsCommentedCode, but those defines are pasted
	// in externally, and will be outside of any directive, and therefore not break this code.
	__m128i k_directive_needle_negated = _mm_setr_epi8(
		0    + 1, '\"' - 1,		// null         to double quote
		'#'  + 1, '\'' - 1,		// hash         to single quote
		'\'' + 1, (char)255,	// single quote to rest of characters -- cast to char to avoid compiler sign warning, but intrinsic uses UBYTE, so code is correct
		0,0,0,0,0,0,0,0,0,0);

	__m128i k_newlines = _mm_set1_epi8('\n');

	for (;;)
	{
		__m128i scan_word = _mm_loadu_si128((const __m128i*)p);
		int newline_mask = _mm_movemask_epi8(_mm_cmpeq_epi8(k_newlines, scan_word));

		// Check for full word with no relevant characters (common case)
		if (!_mm_cmpistrc(k_directive_needle_negated, scan_word, _SIDD_UBYTE_OPS | _SIDD_CMP_RANGES | _SIDD_NEGATIVE_POLARITY))
		{
			line_number += _mm_popcnt_u32(newline_mask);
			p += 16;
			continue;
		}

		// Find the matching character
		int end_index = _mm_cmpistri(k_directive_needle_negated, scan_word, _SIDD_UBYTE_OPS | _SIDD_CMP_RANGES | _SIDD_NEGATIVE_POLARITY);
		newline_mask = newline_mask & (0xffff >> (16 - end_index));
		p += end_index;
		line_number += _mm_popcnt_u32(newline_mask);

		char found_char = *p;
		if (found_char == '#')
		{
			// Scan backwards for a newline.  Writing this non-SSE, with the theory that most directives are at the start of a line,
			// and so testing a single character is all that's required 99% of the time, and an SSE string instruction is much slower
			// for that common case.  We are in a commented out #if block, so there must be a newline related to the #if block earlier
			// in the file, so we don't need to check for beginning of string.
			for (const char* leading_whitespace_scan = p - 1;; leading_whitespace_scan--)
			{
				if (*leading_whitespace_scan == '\n')
				{
					// Leading whitespace hash!  Return result.
					*p_line_number = line_number;
					return p;
				}
				else if (pp_char_class[(uint8)*leading_whitespace_scan] != PP_CHAR_CLASS_whitespace)
				{
					// Not a leading hash!  Continue parsing
					p++;
					break;
				}
			}
		}
		else if (found_char == '\"' || found_char == '\'')
		{
			// Quoted text.  Not bothering to SSE optimize this, as it is rarely reached.  Directives containing text aren't scanned
			// in this loop, and other sources of text are rare.
			char current_char;
			for (p++; (current_char = *p); p++)
			{
				if (current_char == found_char)
				{
					// Found close quote, skip it and break out
					p++;
					break;
				}
				else if (current_char == '\\')
				{
					// Escape, skip the next character unconditionally if not EOF
					if (p[1] != 0)
					{
						p++;
						current_char = *p;
					}
				}

				// Track newlines (including escaped newlines)
				if (current_char == '\n')
				{
					line_number++;
				}
			}

			if (!current_char)
			{
				// Unterminated string
				*p_line_number = line_number;
				return p;
			}
		}
		else
		{
			// Null terminator
			*p_line_number = line_number;
			return p;
		}
	}

#else  // PREPROCESSOR_USE_SSE4_2

	int state = PP_STATE_ready, prev_state;
	for (;;)
	{
		do
		{
			uint8 ch = *p++;
			uint8 ch_class = pp_scan_char_class[ch];  // don't stop at identifiers
			prev_state = state;
			state = pp_transition_table[state][ch_class];
			line_number += pp_state_is_end_of_line[state];
		} while (state < PP_STATE_active_count);

		if (state != PP_STATE_saw_backslash)
		{
			*p_line_number = line_number;
			return p - 1;
		}

		STB_COMMON_ASSERT(p[-1] == '\\');

		// if character after \ is CR/LF, join the lines together
		if (*p == '\n' || *p == '\r')
		{
			p += newline_char_count(p[0], p[1]);  // skip past CR, LF, CR LF, LF CR
			++line_number;
			state = prev_state;	 // reset parse state to before the backslash
		}
		else
			state = PP_STATE_ready;
	}

#endif  // !PREPROCESSOR_USE_SSE4_2
}

//
//  C-style scanners
//

// this is used when parsing macro arguments... so @TODO comments could be present
static const char* preprocessor_skip_whitespace(const char* s, int* newline_count)
{
	for (;;)
	{
		while (char_is_whitespace(*s))
		{
			while (char_is_nonnewline_whitespace(*s))
				++s;

			while (char_is_newline(*s))
			{
				s += newline_char_count(s[0], s[1]);
				++*newline_count;
			}
		}

		// handle backslash continuation
		if (s[0] == '\\' && char_is_newline(s[1]))
		{
			++s;
			s += newline_char_count(s[0], s[1]);
			++*newline_count;
		}
		else
			return s;
	}
}

// this is used when parsing directives, so newlines terminate it
// comments are allowed but rare.
static const char* preprocessor_skip_whitespace_in_directive(const char* s, int* newline_count)
{
	for (;;)
	{
		// loop over whitespace
		while (char_is_nonnewline_whitespace(*s))
			++s;

		if (*s == '/')
		{
			// skip comments
			if (s[1] == '/')
				goto scan_to_end_of_line;
			if (s[1] != '*')
				break;
			// parse multiline comment
			s += 2;
			for (;;)
			{
				if (s[0] == 0)
					break;	// @TODO: error, end of file in multiline comment
				if (s[0] == '*' && s[1] == '/')
				{
					s += 2;
					break;
				}
				++s;
			}
		}
		else if (*s == '\\' && char_is_newline(s[1]))
		{
			++s;
			s += newline_char_count(s[0], s[1]);
			++newline_count;
		}
		else
			return s;
	}
scan_to_end_of_line:
	while (!char_is_end_of_line(*s))
		++s;
	return s;
}

static char* copy_line_without_comments(char* buffer, size_t buffer_size, const char* p, int* line_count, const char** endp)
{
	char *out = buffer, *end = buffer + buffer_size;
	while (char_is_nonnewline_whitespace(*p))
		++p;

	for (;;)
	{
		// fast copy into buffer
		while (out < end && !char_is_copy_line_exception(*p))
			*out++ = *p++;

		// handle end of line & backslash
		if (char_is_end_of_line(*p))
		{
			if (p[0] == 0 || p[-1] != '\\')
			{
				*out = 0;
				*endp = p;
				return buffer;
			}
			// process backslash escape by ungetting the backslash and skipping the newline
			--out;
			p += newline_char_count(p[0], p[1]);
			++*line_count;
		}
		else if (p[0] == '/')
		{
			if (p[1] == '/')
			{
				while (!char_is_end_of_line(*p))
					++p;
				*out = 0;
				*endp = p;
				return buffer;
			}
			else if (p[1] == '*')
			{
				p += 2;
				while (!(p[0] == '*' && p[1] == '/'))
				{
					if (char_is_end_of_line(*p))
					{
						if (*p == 0)
							return buffer;
						p += newline_char_count(p[0], p[1]);
						++*line_count;
					}
					else
						++p;
				}
				p += 2;
			}
			else
			{
				*out++ = *p++;
			}
		}
		else
		{
			STB_COMMON_ASSERT(out >= end);
		}

		if (out >= end)
			break;
	}

	// if we reach here, the copy above overflowed the buffer
	{
		size_t length = out - buffer;
		out = NULL;	 // stb_ds array
		arrinitlen(out, length);
		memcpy(out, buffer, length);

		// this case is most likely to be a long #define
		for (;;)
		{
			size_t count = 0;
			while (!char_is_copy_line_exception(*p))
			{
				++count;
				++p;
			}
			arrsetlen(out, arrlen(out) + count);
			memcpy(out + arrlen(out) - count, p - count, count);

			// handle end of line & backslash
			if (char_is_end_of_line(*p))
			{
				if (p[0] == 0 || p[-1] != '\\')
				{
					arrput(out, 0);
#if SSE_READ_PADDING
					arrsetcap(out, arrlennonull(out) + SSE_READ_PADDING);
#endif
					*endp = p;
					return out;
				}
				// process backslash escape by ungetting the backslash and skipping the newline
				arrsetlen(out, arrlen(out) - 1);
				p += newline_char_count(p[0], p[1]);
				++*line_count;
			}
			else if (p[0] == '/')
			{
				if (p[1] == '/')
				{
					while (!char_is_end_of_line(*p))
						++p;
					arrput(out, 0);
#if SSE_READ_PADDING
					arrsetcap(out, arrlennonull(out) + SSE_READ_PADDING);
#endif
					*endp = p;
					return out;
				}
				else if (p[1] == '*')
				{
					p += 2;
					while (!(p[0] == '*' && p[1] == '/'))
					{
						if (char_is_end_of_line(*p))
						{
							if (*p == 0)
								return buffer;	// @TODO: unterminated multline comment needs to report error
							p += newline_char_count(p[0], p[1]);
							++*line_count;
						}
						else
							++p;
					}
					p += 2;
				}
				else
				{
#pragma warning(push)
#pragma warning(disable:6011) // MSVC analysis doesn't understand that arrmaybegrow can validly take null and allocate (thinks it's a possible null deref)
					arrput(out, *p++);
#pragma warning(pop)
				}
			}
			else
			{
				STB_COMMON_ASSERT(0);
			}
		}
	}
}

// copy until a newline, but with backslash-continuation elimination
static char* convert_backslash_continuation(char* src, char* no_slashes_before_this_point, char** update_src, int* line_number)
{
	char* copy = NULL;
	char* p = src;

	(void)sizeof(no_slashes_before_this_point);	 // @OPTIMIZE faster copy up to this point to avoid rescanning

	// Implementation strategies:
	//    1. scan entire line to find length; alloc; copy; this will do 2n over the input
	//    2. use dynamic array to assemble; this will do n over the input and O(n) copies of the output during realloc
	// But 2 can avoid realloc if the initial size is large enough
	//
	// @OPTIMIZE: faster if we don't use stb_ds but use explicit buffer/capacity/length,
	// so we can keep them in registers.

	arrsetcap(copy, 1024);
	for (;;)
	{
		while (!char_is_end_of_line(*p) && *p != '/')
		{
			arrput(copy, *p);
			++p;
		}
		if (*p == 0)
			break;
		if (*p == '/')
		{
			if (p[1] == '/')
			{
				while (!char_is_end_of_line(*p))
					++p;
				break;
			}
			else if (p[1] == '*')
			{
				p += 2;
				while (*p != 0 && (p[0] != '*' || p[1] != '/'))
					++p;
				p += 2;
				arrput(copy, ' ');	// avoid token pasting
				continue;
			}
			else
			{
				arrput(copy, *p++);
				continue;
			}
		}
		STB_COMMON_ASSERT(char_is_newline(*p));
		// if newline is preceded by backslash, keep going
		if (p[-1] != '\\')
			break;
		p += newline_char_count(p[0], p[1]);
		++*line_number;
		arrsetlen(copy, arrlen(copy) - 1);	// undo the output of the backslash
	}

	arrput(copy, 0);

	if (update_src)
		*update_src = p;

	return copy;
}

static char* maybe_convert_backslash_continuation(char* src, char** update_src, char** alloc_buffer, ptrdiff_t* length, int* line_number)
{
	char* p = src;
	while (!char_is_end_of_line(*p) && *p != '/')
		++p;
	if (*p == 0)
	{
		*length = p - src;
		*update_src = p;
		return src;
	}
	if ((char_is_newline(*p) && p[-1] == '\\') || *p == '/')
	{
		char* alloc = convert_backslash_continuation(src, p - 1, update_src, line_number);
		*alloc_buffer = alloc;
		*length = arrlen(alloc) - 1;  // don't count NUL
		return alloc;
	}
	else
	{
		STB_COMMON_ASSERT(char_is_newline(*p));
		*length = p - src;
		*update_src = p;
		return src;
	}
}

//////////////////////////////////////////////////////////////////////////////
//
//  MAIN PROCESSING LOOP
//

enum
{
	IN_MACRO_no,
	IN_MACRO_yes,
	IN_MACRO_directive,		// e.g. expanding #include or #pragma
	IN_MACRO_if_condition,	// if we're expanding '#if' so we need to do special processing for defined()
	IN_MACRO_include_guard_scan,
};

enum
{
	CONDITIONAL_none,
	CONDITIONAL_skip_current_block,
	CONDITIONAL_skip_all_blocks,
};

typedef struct
{
	int conditionally_disable;
	int conditional_compilation_nesting_level;
} conditional_state;

static void process_directive(parse_state* cs, conditional_state* cons);
static void maybe_expand_macro(parse_state* cs, struct macro_definition* pending, int in_macro_expansion, char* in_macro, struct macro_definition** md);

enum
{
	// perfect hash function: sum of ASCII values mod 32
	HASH_elif = 0,
	HASH_include = 4,
	HASH_endif = 6,
	HASH_line = 8,
	HASH_else = 9,
	HASH_error = 10,
	HASH_define = 11,
	HASH_if = 15,
	HASH_ifndef = 12,
	HASH_undef = 18,
	HASH_pragma = 24,
	HASH_ifdef = 30,
	HASH_warning = 22,

	HASH_none = 31,
};

// normally this comes from a safe buffer, but during disabled-by-#if scanning it's an uncopied directive
static int parse_directive_after_hash(const char** ptr_p, int* newlines)
{
	int sum = 0;
	const char* p = *ptr_p;
	const char* start;
	p = preprocessor_skip_whitespace_in_directive(p, newlines);
	start = p;
	while (char_is_pp_identifier_first(*p))
		sum += (uint8)*p++;
	sum &= (DIRECTIVE_HASH_SIZE - 1);

	if (fast_preprocess_string_equals_short(start, directive_hash[sum].name, directive_hash[sum].name_len) && !char_is_pp_identifier(directive_hash[sum].name_len))
	{
		*ptr_p = p;
		return sum;
	}
	else
	{
		*ptr_p = start;
		return HASH_none;
	}
}

static void preprocess_string(parse_state* cs, int in_macro_expansion, char* in_macro, struct macro_definition** final_function_macro)
{
	pp_context* c = cs->context;
	conditional_state cons;
	cons.conditionally_disable = CONDITIONAL_none;
	cons.conditional_compilation_nesting_level = 0;
	cs->include_guard_candidate_macro_name = 0;

	if (in_macro_expansion == IN_MACRO_include_guard_scan)
	{
		const char* p;
		int action;

		in_macro_expansion = IN_MACRO_no;
		c->include_guard_detect = 1;

		// check for leading #ifndef
		action = scan_whitespace_and_comments(cs);
		if (action != PP_WHITESCAN_STATE_stop_seen_onechar || cs->src[cs->src_offset] != '#')
			goto handle_action;

		p = preprocessor_skip_whitespace_simple(cs->src + cs->src_offset + 1);	// skip past '#'
		if (0 != memcmp(p, "ifndef", 6))
			goto handle_action;

		process_directive(cs, &cons);
		if (c->last_ifndef == 0)
			goto done;

		// check for following #define
		action = scan_whitespace_and_comments(cs);
		if (action != PP_WHITESCAN_STATE_stop_seen_onechar || cs->src[cs->src_offset] != '#')
			goto handle_action;
		p = preprocessor_skip_whitespace_simple(cs->src + cs->src_offset + 1);	// skip past '#'
		if (0 != memcmp(p, "define", 6))
		{
		handle_action:
			// rewind to good point to continue scanning with regular scanner, to keep code simple
			switch (action)
			{
				case PP_WHITESCAN_STATE_stop_seen_onechar:
					rewind_to_start_of_line(cs);
					break;
				case PP_WHITESCAN_STATE_stop_seen_twochars:
					rewind_to_start_of_line(cs);
					break;
				case PP_WHITESCAN_STATE_stop_eof:
					break;
				case PP_WHITESCAN_STATE_error_unterminated_multiline_comment:
					goto unterminated_multline_comment;
			}
		done:;
		}
		else
		{
			process_directive(cs, &cons);
			if (c->last_macro_definition && 0 == strcmp(c->last_macro_definition->symbol_name, c->last_ifndef))
			{
				cs->include_guard_candidate_macro_name = c->last_ifndef;
			}
		}
		c->include_guard_detect = 0;
	}

	for (;;)
	{
		// char *start = cs->src + cs->src_offset; // use this to rediscover the start of an unterminated /* comment
		int action;

		// only output line directives at start of line
		if (cs->in_leading_whitespace && in_macro_expansion == IN_MACRO_no)
		{
			if (cs->src_line_number != cs->dest_line_number || cs->last_output_filename != cs->filename)
			{
				if (cs->last_output_filename == cs->filename && cs->dest_line_number < cs->src_line_number && cs->src_line_number < cs->dest_line_number + 12)
				{
					int n = cs->src_line_number - cs->dest_line_number;
					while (cs->dest_line_number < cs->src_line_number)
					{
						arrput(cs->dest, '\n');
						++cs->dest_line_number;
					}
				}
				else
					output_line_directive(cs);
			}
		}

		action = in_macro_expansion > IN_MACRO_no ? copy_to_action_point_macro_expansion(cs) : copy_to_action_point(cs);

		STB_COMMON_ASSERT(cons.conditional_compilation_nesting_level >= 0);

		switch (action)
		{
			case PP_STATE_saw_leading_hash:
			{
				MDBG();
				process_directive(cs, &cons);
				if (c->stop)
					return;
				cs->in_leading_whitespace = 1;
				MDBG();
				break;
			}

			case PP_STATE_saw_identifier_start:
			{
				struct macro_definition* md = 0;
				maybe_expand_macro(cs, NULL, in_macro_expansion, in_macro, &md);
			iterate_final_macro:
				if (c->stop)
					return;
				cs->in_leading_whitespace = 0;
				if (md != NULL)
				{
					// we saw a function-like macro name at the end of the macro expansion above, but no (.
					// so we now check to see whether it's followed by (. clang looks inside following macros
					// for this, but VC6 doesn't, and it would require a rearchitecture to look inside the macros
					// (and probably an efficiency hit), so we'll do it the VC6 way.
					const char* s = cs->src + cs->src_offset;
					// @TODO skip comments and newlines
					while (char_is_nonnewline_whitespace(*s))
						++s;
					cs->src_offset = s - cs->src;
					if (*s == '(')
					{
						struct macro_definition* temp = md;
						md = NULL;
						maybe_expand_macro(cs, temp, in_macro_expansion, in_macro, &md);
						goto iterate_final_macro;
					}
					else
					{
						if (*s == 0 && final_function_macro != NULL)
						{
							*final_function_macro = md;
							return;
						}
					}
					// the final macro wasn't copied into the output destination yet, so do it now
					{
						size_t len = strlen(md->symbol_name);
						size_t off = arraddnindex(cs->dest, len);
						memcpy(cs->dest + off, md->symbol_name, len);
					}
				}
				break;
			}

			case PP_STATE_saw_hash:
			{
				if (do_error_code(cs, PP_RESULT_directive_not_at_start_of_line, "Preprocessor directive must appear at beginning of line."))
				{
					return;
				}
				else
				{
					int lines, hash;
					// scan and hash the directive name (p initialized to the character after the #)
					const char* p = cs->src + 1;
					hash = parse_directive_after_hash(&p, &lines);
					if ((hash & (DIRECTIVE_HASH_SIZE - 1)) == HASH_pragma)
					{
						// allow (and process) #pragmas in #defines
						process_directive(cs, &cons);
					}
					else
					{
						// copy all other directives (or other usage of #, i.e. HLSL infinity constant 1.#INF) through to the output
						arrput(cs->dest, '#');
						++cs->src_offset;
					}
				}
				break;
			}

			case PP_STATE_eof:
			{
				if (cons.conditional_compilation_nesting_level)
				{
					int i, stop = 0;
					for (i = (int)arrlen(c->ifdef_stack) - 1; i >= 0; --i)
					{
						if (do_error(cs, "No #endif found for previous #%s", &directive_hash[c->ifdef_stack[i].type].name[0]))
							stop = 1;
						error_supplement(cs, PP_RESULT_ERROR, c->ifdef_stack[i].line, "Location of #%s", &directive_hash[c->ifdef_stack[i].type].name[0]);
					}
					if (stop)
						return;
				}
				return;
			}

			unterminated_multline_comment:
			case PP_STATE_error_unterminated_multiline_comment:
			{
				// @TODO: locate the comment's starting location, which requires fully rescanning
				// from 'start'
				do_error(cs, "Unclosed '/*' comment.");
				return;
			}

			default:
				STB_COMMON_ASSERT(0);
				break;
		}

		MDBG();
	}
}

//////////////////////////////////////////////////////////////////////////////
//
//  MACRO PROCESSING
//

#define MAX_MACRO_PARAMETER_NAME_LENGTH 256

static void update_macro_filter(pp_context* c, char* identifier, size_t identifier_length)
{
	STB_COMMON_ASSERT(identifier[0] != 0 && identifier_length > 0);

	size_t bloom_filter_offset = identifier_length < MACRO_BLOOM_FILTER_SIZE ? identifier_length : MACRO_BLOOM_FILTER_SIZE - 1;

	// Set bits for three key characters of the macro by length (first, middle, last)
	c->macro_bloom_filter[bloom_filter_offset][0] |= (1ull << (identifier[0] & 0x3f));
	c->macro_bloom_filter[bloom_filter_offset][1] |= (1ull << (identifier[identifier_length >> 1] & 0x3f));
	c->macro_bloom_filter[bloom_filter_offset][2] |= (1ull << (identifier[identifier_length - 1] & 0x3f));
}

static struct macro_definition* create_macro_definition(parse_state* ps, const char* def)
{
	// keep track of the state so we can share the diagnostic code
	enum
	{
		STATE_pre_name,
		STATE_object_like,
		STATE_function_like
	} state = STATE_pre_name;

	pp_context* c = ps->context;
	char identifier_buffer[MAX_MACRO_PARAMETER_NAME_LENGTH + 1];
	struct macro_definition* m = (struct macro_definition*)stb_arena_alloc(&c->macro_arena, sizeof(*m));
	const char* start = preprocessor_skip_whitespace_simple(def);
	const char* p;

	m->symbol_name = NULL;
	m->disabled = 0;
	m->predefined = 0;
	m->expansion = NULL;

	// parse identifier
	p = start;
	if (!char_is_pp_identifier_first(*p))
		goto invalid_char_error;
	while (char_is_pp_identifier(*p))
		++p;

	m->symbol_name = stb_arena_alloc_string_length(&c->macro_arena, start, p - start);
	m->symbol_name_length = (int)(p - start);

	if (*p != '(')
	{
		state = STATE_object_like;

		// object-like definition
		if (!char_is_whitespace_or_eof(*p))
			goto invalid_char_error;

		// find the start of the definition
		p = preprocessor_skip_whitespace_simple(p);

		start = p;

		while (!char_is_end_of_line(*p))
			++p;

		// eliminate trailing whitespace
		if (p > start)
		{
			while (char_is_whitespace(p[-1]))
				--p;
			STB_COMMON_ASSERT(p > start);
		}

		m->num_parameters = MACRO_NUM_PARAMETERS_no_parentheses;
		size_t expansion_length = p - start;
		if (*(start + (expansion_length - 1)) == '>') // hacky special case in macro expansion to work around DXC bug (not handling >> in template declarations)
		{
			m->simple_expansion = arena_alloc_string_trailingspace(&c->macro_arena, start, expansion_length);
			m->simple_expansion_length = (int)expansion_length + 1;
		}
		else
		{
			m->simple_expansion = stb_arena_alloc_string_length(&c->macro_arena, start, expansion_length);
			m->simple_expansion_length = (int)(expansion_length);
		}
	}
	else
	{
		// function-like definition
		pphash parameters;
		int token_paste_before_text = 0;
		state = STATE_function_like;
		m->num_parameters = 0;
		m->is_variadic = 0;
		STB_COMMON_ASSERT(*p == '(');
		// in the () case, we can do simple non-function-like expansion, but to share
		// code, we use this same path, but that means in the () case we still need to
		// allocate the parameters dictionary so it can return -1 as the default

		stringhash_create(&parameters, 64);	 // oversized so hash table is faster

		p = preprocessor_skip_whitespace_simple(p + 1);	 // skip paren then following whitespace
		// accept either ) or C Identifier or ...
		if (*p == ')')
		{
			;  // do nothing, we have defined as having 0 parameter
		}
		else
		{
			if (*p == '.')
			{
				if (p[1] == '.' && p[2] == '.')
				{
					p = preprocessor_skip_whitespace_simple(p + 3);
					if (*p != ')')
						goto invalid_char_error;
					m->is_variadic = 1;
					stringhash_put(&parameters, "__VA_ARGS__", 11, (void*)(size_t)1);
				}
				else
					goto invalid_char_error;
			}
			else if (char_is_pp_identifier_first(*p))
			{
				// parse the parameters
				for (;;)
				{
					start = p = preprocessor_skip_whitespace_simple(p);
					if (!char_is_pp_identifier_first(*p))
					{
						if (p[0] == '.' && p[1] == '.' && p[2] == '.')
						{
							p = preprocessor_skip_whitespace_simple(p + 3);
							if (*p != ')')
								goto invalid_char_error;
							m->is_variadic = 1;
							stringhash_put(&parameters, "__VA_ARGS__", 11, (void*)((size_t)m->num_parameters + 1));
							break;
						}
						else
						{
							goto invalid_char_error;
						}
					}
					else
					{
						size_t length;
						++p;
						while (char_is_pp_identifier(*p))
							++p;
						length = p - start;

						if (stringhash_get(&parameters, start, length) != 0)
						{
							if (length > sizeof(identifier_buffer) - 1)
								length = sizeof(identifier_buffer) - 1;
							memcpy(identifier_buffer, start, length);
							identifier_buffer[length] = 0;
							do_error(ps, "Macro '%s' contained the parameter '%s' more than once", m->symbol_name, identifier_buffer);
							goto error_exit;
						}
						stringhash_put(&parameters, start, length, (void*)(size_t)++m->num_parameters);
						p = preprocessor_skip_whitespace_simple(p);
						if (*p == ')')
							break;	// dont with parameter loop
						if (*p == ',')
						{
							++p;
							continue;
						}
						goto invalid_char_error;
					}
				}
			}
			else if (*p == 0)
			{
				goto invalid_char_error;
			}
		}

		STB_COMMON_ASSERT(*p == ')');
		p = preprocessor_skip_whitespace_simple(p + 1);

		// now the remainder of p is the substitution string. we need to parse it
		// for identifiers that are in the parameters table
		m->expansion = NULL;
		token_paste_before_text = 0;  // @TODO: this used to be 1 to make sure we didn't generate a leading space, but this was wrong with reparsing. but may be
									  // problematic with nesting now?
		for (;;)
		{
			int found_parameter = -1;
			int require_identifier = 0;
			int stringify = 0;
			int token_paste_after_text = 0;
			expansion_part e;
			const char* end;
			start = p;
			// we need to parse to an identifier or '#', and we need to
			// skip comments and string literals. we maybe could speed
			// this up with a scanner, but we probably don't spend that
			// much time here anyway
			for (;;)
			{
				if (char_is_quote(*p))
				{
					char ch = *p++;
					while (*p && *p != ch)
						p += (*p == '\\' ? 2 : 1);
					if (*p == ch)
						++p;
				}
				else if (char_is_pp_identifier_first(*p))
				{
					size_t len;
					const char* s = scan_pp_identifier(p);
					len = s - p;
					found_parameter = ((int)(size_t)stringhash_get(&parameters, p, len)) - 1;
					if (found_parameter >= 0)
					{
						end = p;
						p = s;
						break;
					}
					p = s;
				}
				else if (*p == '#')
				{
					end = p;
					if (p[1] == '#')
					{
						require_identifier = 1;
						token_paste_after_text = 1;
						p = preprocessor_skip_whitespace_simple(p + 2);
						if (*p == '#')
						{
							stringify = 1;
							++p;
						}
						break;
					}
					require_identifier = 1;
					stringify = 1;
					++p;
					break;
				}
				else if (char_is_end_of_line(*p))
				{
					end = p;
					break;
				}
				else
					++p;
			}
			if (char_is_end_of_line(*p) && found_parameter < 0)
			{
				e.argument_index = -1;
				while (start < p && char_is_whitespace(*start))
					++start;
				e.text = arena_alloc_padded_string(&c->macro_arena, start, p - start);
				e.text_length = (uint32)(p - start + 2);
				// always trim final space
				e.text[--e.text_length] = 0;
				if (token_paste_before_text)
				{
					++e.text;  // delete space at front so token-pasting occurs
					--e.text_length;
				}
				arrput(m->expansion, e);
				break;
			}
			if (found_parameter < 0)
			{
				size_t len;
				const char* s;
				STB_COMMON_ASSERT(require_identifier);
				p = preprocessor_skip_whitespace_simple(p);
				if (!char_is_pp_identifier_first(*p))
					goto expected_parameter_name;
				s = scan_pp_identifier(p);
				len = s - p;
				found_parameter = ((int)(size_t)stringhash_get(&parameters, p, len)) - 1;
				p = s;
				if (found_parameter < 0)
					goto expected_parameter_name;
			}

			while (char_is_whitespace(*start) && start < end)
				++start;
			while (end - 1 > start && char_is_whitespace(end[-1]))
				--end;

			// if the string is empty, then a token paste is between two parameters
			if (start == end && token_paste_before_text)
				token_paste_after_text = 1;

			e.argument_index = (int16)found_parameter;
			e.stringify_argument = (uint8)stringify;
			e.concat_argument = (uint8)token_paste_after_text;
			e.eliminate_comma_before_empty_arg = 0;
			e.text = arena_alloc_padded_string(&c->macro_arena, start, end - start);
			e.text_length = (uint32)(end - start + 2);

			// to token paste with following parameter, remove space at end of expansion text
			if (token_paste_after_text)
			{
				e.text[--e.text_length] = 0;
			}

			// to token paste with previous parameter, remove space at start of expansion text
			if (token_paste_before_text)
			{
				++e.text;
				e.text_length -= 1;
			}

			arrput(m->expansion, e);

			// check if the argument is followed by ##
			p = preprocessor_skip_whitespace_simple(p);
			if (p[0] == '#' && p[1] == '#')
			{
				// if so, we want to token paste between this argument and the next text, i.e. before the next text
				token_paste_before_text = 1;
				arrlast(m->expansion).concat_argument = 1;
				p += 2;
			}
			else
				token_paste_before_text = 0;
		}

		stringhash_destroy(&parameters);
	}

	return m;

error_exit:
{
	return NULL;
}

invalid_char_error:
{
	char* error_messages[3] = {
		"Invalid character %s after '#define'",
		"Invalid character %s in macro definition after '%s'",
		"Invalid character %s in parameter list for macro definition '%s'",
	};

	do_error_char_code(ps, PP_RESULT_invalid_char, error_messages[state], *p, m->symbol_name);

	return NULL;
}

expected_parameter_name:
{
	do_error(ps, "Expected a parameter name to follow # character in definition of macro '%s'.", m->symbol_name);

	return NULL;
}
}

#if 0
// originally, we tried to use a fast argument scanner that didn't make a copy.
// however, we need to append a NUL to the end of the argument so that we can
// feed it to the macro scanner, and arguments can come from the input file.
// (we can't overwrite that in the middle with the NUL since it will be memory-mapped
// in the common case)
//
// so instead we always copy the argument. might be worth coming back to this and
// seeing if it's better to never memory map and instead apply this logic!
static char *parse_argument(char *text, int *line_number, int *needs_copying)
{
  int num_parens=0;
  char *p = text;
  for(;;) {
    while (!char_is_argument_parse_action((uint8) *p))
      ++p;
    switch (*p) {
      case '\n': case '\r':
        p += newline_char_count(p[0],p[1]);
        ++*line_number;
        break;
      case ',':
        if (num_parens == 0)
          return p;
        ++p;
        break;
      case ')':
        if (num_parens == 0)
          return p;
        --num_parens;
        ++p;
        break;
      case '(':
        ++num_parens;
        ++p;
        break;
      case '\'':
      case '\"': {
        char ch = *p++;
        for(;;) {
          while (!char_is_special_string_contents(*p))
            ++p;
          if (*p == ch || *p == 0)
            break;
          if (*p == '\\')
            ++p; // skip character after backslash in case it's a quote mark
          ++p;
        }
        if (*p == 0) {
          return p;
        }
        STB_COMMON_ASSERT(*p == ch);
        ++p;
        break;
      }
      case '\\':
        *needs_copying = 1;
        return p;
      case '/':
        if (p[1] == '/' || p[1] == '*') {
          *needs_copying = 1;
          return p;
        } else
          p += 1;
        break;
      default:
        STB_COMMON_ASSERT(0);
      case 0:
        return p;
    }
  }
}
#endif

static const char* copy_argument(const char* text, int* line_number, char** p_out)
{
	int num_parens = 0;
	const char* p = text;
	char* out = *p_out;	 // need to be able to append to existing buffer for VA_ARGS
	arrsetcap(out, 200);

	for (;;)
	{
		// scan characters that don't need processing
		// we don't have a length bound, so we don't copy as we go, we make a second pass
		const char* q = p;
		ptrdiff_t addlen;
		size_t oldlen;
		while (!char_is_argument_parse_action((uint8)*p))
		{
			++p;
		}
		oldlen = arrlennonull(out);
		addlen = p - q;
		STB_ASSUME(out != NULL && oldlen > 0);
		arrsetlen(out, oldlen + addlen);
		memcpy(out + oldlen, q, addlen);

		switch (*p)
		{
			case '\n':
			case '\r':
				if (p[-1] == '\\')
					arrsetlen(out, arrlen(out) - 1);  // undo output of '\'
				else
					arrput(out, '\n');
				p += newline_char_count(p[0], p[1]);
				++*line_number;
				break;
			case ',':
				if (num_parens == 0)
				{
					*p_out = out;
					return p;
				}
				arrput(out, *p);
				++p;
				break;
			case ')':
				if (num_parens == 0)
				{
					*p_out = out;
					return p;
				}
				--num_parens;
				arrput(out, *p);
				++p;
				break;
			case '(':
				++num_parens;
				arrput(out, *p);
				++p;
				break;
			case '\'':
			case '\"':
			{
				char ch = *p++;
				arrput(out, ch);
				for (;;)
				{
					while (!char_is_special_string_contents(*p))
					{
						arrput(out, *p);
						++p;
					}
					if (*p == ch || *p == 0)
						break;
					if (*p == '\\')
					{
						arrput(out, *p);
						++p;
					}
					arrput(out, *p);
					++p;
				}
				if (*p == 0)
				{
					return p;
				}
				STB_COMMON_ASSERT(*p == ch);
				arrput(out, *p);
				++p;
				break;
			}
			case '/':
				if (p[1] == '/')
				{
					while (!char_is_end_of_line(*p))
						++p;
					if (*p == 0)
						return p;
					p += newline_char_count(p[0], p[1]);
					arrput(out, '\n');
					++*line_number;
				}
				else if (p[1] == '*')
				{
					p += 2;
					while (p[0] && !(p[0] == '*' && p[1] == '/'))
					{
						if (*p == '\n' || *p == '\r')
						{
							++*line_number;
							p += newline_char_count(p[0], p[1]);
							arrput(out, '\n');
						}
						else
						{
							++p;
						}
					}
					if (*p == 0)
						return p;
					p += 2;
					arrput(out, ' ');
				}
				else
				{
					arrput(out, *p);
					p += 1;
				}
				break;
			default:
				STB_COMMON_ASSERT(0);
			case 0:
			{
				*p_out = out;
				// allow caller to handle the error
				return p;
			}
		}
	}
}

// if the identifier is a function-type macro but it is followed by EOF, then the parent needs to check
// if it's followed by '('
static void maybe_expand_macro(parse_state* cs, struct macro_definition* pending, int in_macro_expansion, char* in_macro, struct macro_definition** final_function_macro)
{
	pp_context* c = cs->context;
	struct macro_definition* md;
	char* identifier = (char*)cs->dest + arrlennonull(cs->dest);
	size_t identifier_length;
	const char* p = cs->src + cs->src_offset;
	char* q = identifier;

	if (pending)
	{
		md = pending;
		identifier_length = 0;	// NOTUSED
	}
	else
	{
		// Identifier will have been copied and length set by copy_and_filter_macro
		identifier_length = cs->copied_identifier_length;
		cs->copied_identifier_length = 0;

		// "maybe_expand_macro" consumes identifiers originally copied by "copy_and_filter_macro" -- make sure one was copied
		STB_COMMON_ASSERT(identifier_length > 0);

		p += identifier_length;
		q += identifier_length;

		md = (struct macro_definition*)stringhash_get(&c->macro_map, identifier, identifier_length);
		if (md == NULL || md->disabled)
		{
		not_macro:
			cs->src_offset = p - cs->src;
			arrsetlen(cs->dest, (size_t)(q - cs->dest));  // set the output buffer size to account for the above-copied identifier
			return;
		}

		cs->src_offset = p - cs->src;
	}

	if (md->num_parameters < 0)
	{
		// no parens, no substitutions, so just process the expansion without copying

		switch (md->num_parameters)
		{
			case MACRO_NUM_PARAMETERS_no_parentheses:
			{
				if (md->simple_expansion_length == 0)
					// normally we avoid token pasting by pre-adding spaces to expansion. but if client
					// passes in their own macro definitions, we let them pass truly empty strings,
					// and prevent token pasting here; otherwise e.g.
					//   foo.c:              +FOO+x;
					//   command:            cpp -DFOO foo.c
					//   desired output:     + +x;
					//   without next line:  ++x;
					arrput(cs->dest, ' ');
				else
				{
					parse_state ncs = *cs;
					ncs.parent = cs;
					ncs.src = md->simple_expansion;
					ncs.src_offset = 0;
					ncs.src_length = md->simple_expansion_length;
					md->disabled = 1;
					preprocess_string(&ncs, IN_MACRO_yes, md->symbol_name, final_function_macro);
					md->disabled = 0;
					cs->dest = ncs.dest;
				}
				break;
			}
			case MACRO_NUM_PARAMETERS_defined:
			{
				int parenthesized = 0;
				size_t len;

				// copy through the following token as well
				while (char_is_whitespace(*p))
					++p;
				if (*p == '(')
				{
					++p;
					parenthesized = 1;
					while (char_is_whitespace(*p))
						++p;
				}
				if (!char_is_pp_identifier_first(*p))
				{
					do_error(cs, "expected macro name after keyword 'defined'");
				}
				// rewind the output and copy the next identifier to the output temporarily
				q = identifier;
				do
					*q++ = *p++;
				while (char_is_pp_identifier(*p));
				*q = 0;
				len = q - identifier;

				// rewind the output to prepare to write 0 or 1
				q = identifier;
				if (stringhash_get(&c->macro_map, identifier, len))
					*q++ = '1';
				else
					*q++ = '0';

				if (parenthesized)
				{
					while (char_is_whitespace(*p))
						++p;
					if (*p != ')')
					{
						do_error_char(cs, "unexpected character %s at end of 'defined('", *p);
						return;
					}
					++p;
					*q++ = ' ';	 // avoid token-pasting
				}
				cs->src_offset = p - cs->src;
				arrsetlen(cs->dest, (size_t)(q - cs->dest));  // set the output buffer size to account for the above-copied identifier
				break;
			}

			case MACRO_NUM_PARAMETERS_file:
			{
				size_t len = strlen(cs->filename);
				// @TODO: backslash escape filename and cache it in stack, ready for string printing... also use when printing the line number?
				size_t off = arraddnindex(cs->dest, len + 2);
				cs->dest[off] = '"';
				memcpy(cs->dest + off + 1, cs->filename, len);
				cs->dest[off + 1 + len] = '"';
				break;
			}
			case MACRO_NUM_PARAMETERS_line:
			{
				int len;
				q = identifier;
				arrsetcap(cs->dest, arrlen(cs->dest) + INT_STR_SIZE);
				len = snprintf(cs->dest + arrlen(cs->dest), INT_STR_SIZE, "%d", cs->src_line_number);
				arrsetlen(cs->dest, arrlen(cs->dest) + len);
				break;
			}
			case MACRO_NUM_PARAMETERS_counter:
			{
				if (c->counter == 2147483647)
				{
					// we stop after 31-bits just so we don't have to deal with printing more than 10 digits
					do_error_code(cs, PP_RESULT_counter_overflowed, "__COUNTER__ was used too many times (%u)", c->counter);
					return;
				}
				int len;
				q = identifier;
				arrsetcap(cs->dest, arrlen(cs->dest) + INT_STR_SIZE);
				len = snprintf(cs->dest + arrlen(cs->dest), INT_STR_SIZE, "%d", c->counter);
				arrsetlen(cs->dest, arrlen(cs->dest) + len);
				++c->counter;
				break;
			}
			case MACRO_NUM_PARAMETERS_custom:
			{
				while (char_is_whitespace(*p))
					++p;

				// If no parentheses or the macro is disabled (say because we are encountering it nested), just leave the echoed identifier
				if (*p != '(' || md->disabled)
				{
					arrsetlen(cs->dest, (size_t)(q - cs->dest));  // set the output buffer size to account for the above-copied identifier
					return;
				}
				++p;

				// Generate a buffer and append the rest of the arguments to it
				int arg_newlines = 0;
				char* custom_macro_buffer = 0;
				arrsetcap(custom_macro_buffer, identifier_length + 512);
				arrsetlen(custom_macro_buffer, identifier_length + 1);
				memcpy(custom_macro_buffer, identifier, identifier_length);
				custom_macro_buffer[identifier_length] = '(';

				for (;;)
				{
					p = copy_argument(p, &arg_newlines, &custom_macro_buffer);
					if (*p == 0)
					{
						arrfree(custom_macro_buffer);
						do_error(cs, "End-of-file in macro '%s' argument list", md->symbol_name);
						return;
					}

					// Add the comma or close parentheses to macro buffer
					arrput(custom_macro_buffer, *p);
					if (*p++ == ')')
					{
						break;
					}
				}

				// Null terminate
				arrput(custom_macro_buffer, 0);

				// Optionally preprocess the macro args before calling the custom macro callback.  Need to disable the macro first.
				md->disabled = 1;

				if (md->preprocess_args_first)
				{
					parse_state ncs = *cs;
					ncs.parent = cs;
					ncs.src = custom_macro_buffer;
					ncs.src_offset = 0;
					ncs.src_length = arrlennonull(custom_macro_buffer) - 1;
					ncs.dest = 0;
					arrsetcap(ncs.dest, ncs.src_length);
					// Create new dest string
					preprocess_string(&ncs, IN_MACRO_yes, md->symbol_name, NULL);

					// Replace custom_macro_buffer with new dest string
					arrfree(custom_macro_buffer);
					custom_macro_buffer = ncs.dest;
					
					if (!custom_macro_buffer)
					{
						md->disabled = 0;
						do_error(cs, "Preprocessing args for custom macro handler '%s' failed", md->symbol_name);
						return;
					}
					arrput(custom_macro_buffer, 0);
				}

				// Send the custom macro text to the callback, preprocess it, then signal the end of the custom macro
				const char* substitution_text = custommacro_begin(custom_macro_buffer, c->custom_context);

				if (substitution_text == 0)
				{
					md->disabled = 0;
					arrfree(custom_macro_buffer);
					do_error(cs, "Custom macro handler for '%s' failed", md->symbol_name);
					return;
				}

				parse_state ncs = *cs;
				ncs.parent = cs;
				ncs.src = substitution_text;
				ncs.src_offset = 0;
				ncs.src_length = strlen(substitution_text);
				preprocess_string(&ncs, IN_MACRO_yes, md->symbol_name, NULL);
				md->disabled = 0;
				cs->dest = ncs.dest;

				custommacro_end(custom_macro_buffer, c->custom_context, substitution_text);
				arrfree(custom_macro_buffer);

				// Set source offset to the end of the macro
				cs->src_offset = p - cs->src;
				cs->src_line_number += arg_newlines;
				cs->dest_line_number += arg_newlines;
				break;
			}
		}
	}
	else
	{
		// macro has arguments, needs parenthesis
		parse_state ncs = {0}, acs = {0};
		char** arguments = NULL;
		int i;
		int temp_lines;
		int parse_rest = 0;
		const char* s = p;

		stbds_arrinline(arguments, char*, 16);

		// plan:
		//   1. find opening parenthesis, if any
		//   2. parse arguments into arguments[] array
		//   3. parse __VA_ARGS__ and put at end of arguments[] array
		//   4. build temporary buffer containing macro output
		//   5. preprocess temporary buffer

		//////////////////////////////////////////////////////////////////
		//
		//   1. find opening parenthesis, if any
		//

		temp_lines = 0;
		s = preprocessor_skip_whitespace(s, &temp_lines);

		// if macro name appears without following parenthesis, pass it through unaltered
		if (*s != '(')
		{
			if (*s == 0 && in_macro_expansion == IN_MACRO_yes && final_function_macro)
			{
				// except if it's the last thing in a macro expansion
				// skip everything we've seen
				cs->src_offset = s - cs->src;
				*final_function_macro = md;
				return;
			}
			// @OPTIMIZE: don't reparse what we just scanned
			goto not_macro;
		}
		cs->src_line_number += temp_lines;

		// consume all the scanned characters
		p = s + 1;

#if 0
	// WHAT?!?
	// undo '(' from output
	arrsetlen(cs->dest, arrlen(cs->dest)-1);
#endif

		//////////////////////////////////////////////////////////////////
		//
		//   2. parse arguments into arguments[] array
		//

		// Inline storage shared for all argument strings.  Uses stbds_arrinline_suballoc to reuse remaining portion of buffer.
		char* argument_buffer;
		stbds_arrinline(argument_buffer, char, 4096);

		for (i = 0; i < md->num_parameters; ++i)
		{
			char* copy = argument_buffer;
			int arg_newlines = 0;
			const char* leading_ws = p;
			p = preprocessor_skip_whitespace(p, &arg_newlines);
			int leading_ws_size = p - leading_ws;

			s = copy_argument(p, &arg_newlines, &copy);

			// examine character after argument to make sure it's right character
			// the only thing that should have stopped argument parsing is ',' ')' or EOF
			if (*s == ',')
			{
				if (i < md->num_parameters - 1)
					;  // ok, more arguments expected
				else if (md->is_variadic)
					;  // ok, parsed final argument but macro is variadic
				else
				{
					int err = do_error_code(cs, PP_RESULT_too_many_arguments_to_macro, "Too many arguments supplied to macro '%s'; only %d expected",
											md->symbol_name, md->num_parameters);
					if (err)
						return;
					parse_rest = 1;
				}
			}
			else if (*s == ')')
			{
				if (i < md->num_parameters - 1)
				{
					do_error(cs, "Too few arguments supplied to macro '%s'; got %d, expected %d", md->symbol_name, i, md->num_parameters);
					return;
				}
			}
			else
			{
				STB_COMMON_ASSERT(*s == 0);
				if (in_macro_expansion == IN_MACRO_if_condition || in_macro_expansion == IN_MACRO_directive)
					do_error(cs, "Macro '%s' argument list unterminated in %s", md->symbol_name, in_macro);
				else if (in_macro)
					do_error(cs, "Macro '%s' argument list unterminated in expansion of outer macro '%s'", md->symbol_name, in_macro);
				else
					do_error(cs, "End-of-file in macro '%s' argument list", md->symbol_name);
				return;
			}

			{
				char* e;
				// if this is a multi-line argument maintain the whitespace by re-inserting the leading ws (this assumes multiline macro arguments are not token pasted)
				if (arg_newlines > 0)
				{
					arrinsn(copy, 0, leading_ws_size); //-V568
					memcpy(copy, leading_ws, leading_ws_size);

				}
				arrput(copy, 0); // add null terminator to working array

				p = copy;
				e = copy + arrlen(copy) - 1;  // get address of null terminator
				ptrdiff_t len = e - p;			  // get length excluding null terminator
				
				// if this is a single line argument, need to strip trailing whitespace so token-pasting works
				if (arg_newlines == 0)
				{
					if (e - 1 > p)
					{
						e = (char*)preprocessor_skip_whitespace_reverse(e);
						STB_COMMON_ASSERT(e > p);
					}
					*e = 0;
					len = e - p; // reset new length after stripping trailing ws
				}

#if SSE_READ_PADDING
				// Need to ensure padding for safe SSE reads without special cases -- normally this comes from a stack allocated buffer with plenty
				// of padding, so it won't need to reallocate, but we need to handle edge cases.  We need to check padding before the
				// stbds_arrinline_suballoc_char call, which trims the allocation.
				int copy_len = arrlennonull(copy);
				int copy_capacity_padding = arrcapnonull(copy) - copy_len;
#endif

				// Sub-allocate the remaining space in argument_buffer for subsequent arguments.  Note that it's necessary to do the suballocation BEFORE
				// the call to arrsetlen below, because downstream code assumes there is a null terminator beyond the official length of the array.  Doing
				// the sub-allocation first preserves the null terminator -- without this, the sub-allocation will overwrite it.  We only suballocate if
				// a minimum of 200 characters is available, as copy_argument reserves this.
				stbds_arrinline_suballoc_char(argument_buffer, 200);

#if SSE_READ_PADDING
				if (copy_capacity_padding < SSE_READ_PADDING)
				{
					arrsetcap(copy, copy_len + SSE_READ_PADDING);
				}
#endif

				arrsetlen(copy, len); // set argument string len to exclude null terminator
				arrput(arguments, copy);
				cs->src_line_number += arg_newlines;
				cs->dest_line_number += arg_newlines;
				p = s + 1;	// advance to after terminating character
			}
		}

		if (md->num_parameters == 0)
		{
			STB_COMMON_ASSERT(*p == ')' || md->is_variadic);
			s = p;
		}

		if (c->diagnostics && c->stop)
		{
			arrfree(arguments);
			return;
		}

		//////////////////////////////////////////////////////////////////
		//
		//   3. parse __VA_ARGS__ and put at end of arguments[] array
		//

		if (md->is_variadic || parse_rest)
		{
			// create n+1th argument
			// check if it was already terminated with ')'
			if (p[-1] == ')')
			{
				// empty argument
				arrput(arguments, NULL);
			}
			else
			{
				int arg_newlines = 0;
				char *e, *copy = argument_buffer;
				p = preprocessor_skip_whitespace(p, &cs->src_line_number);
				// parse all the remaining arguments
				s = p;
				for (;;)
				{
					arg_newlines = 0;
					s = copy_argument(s, &arg_newlines, &copy);
					if (*s == 0)
					{
						if (in_macro_expansion == IN_MACRO_if_condition || in_macro_expansion == IN_MACRO_directive)
							do_error(cs, "Macro '%s' argument list unterminated in %s", md->symbol_name, in_macro);
						else if (in_macro)
							do_error(cs, "Macro '%s' argument list unterminated in expansion of outer macro '%s'", md->symbol_name, in_macro);
						else
							do_error(cs, "End-of-file in macro '%s' argument list", md->symbol_name);
						return;
					}
					if (*s == ')')
						break;
					arrput(copy, *s);
					++s;
				}

				// s points to closing parenthesis
				STB_COMMON_ASSERT(*s == ')');

				arrput(copy, 0);
				p = copy;
				e = copy + arrlen(copy);

				// strip trailing whitespace
				while (char_is_whitespace(e[-1]))
					--e;
				if (e < copy)
					e = copy;
				*e = 0;

#if SSE_READ_PADDING
				// Need to ensure padding for safe SSE reads without special cases -- normally this comes from a stack allocated buffer with
				// plenty of padding, so this will have sufficient padding, but we need to handle edge cases.
				int copy_len = arrlennonull(copy);
				if (arrcapnonull(copy) - copy_len < SSE_READ_PADDING)
				{
					arrsetcap(copy, copy_len + SSE_READ_PADDING);
				}
#endif

				arrput(arguments, copy);
				cs->src_line_number += arg_newlines;
				cs->dest_line_number += arg_newlines;
			}
		}

		// update the parse_state
		cs->src_offset = (s + 1) - cs->src;

		//////////////////////////////////////////////////////////////////
		//
		//   4. build temporary buffer containing macro output:
		//
		//       for each block of raw text, just copy it
		//       for each stringified or concatenated argument just copy it
		//       for other arguments, perform an expansion now
		//

		MDBG();

		// once we build the concatenated temp buffer, we will rescan that.
		// so that temp buffer is stored in 'ncs'
		ncs = *cs;	// inherit file & line and destination
		ncs.parent = cs;

		// each argument will be parsed using acs, so set that up now
		acs = *cs;
		acs.dest = 0;

		// populate temporary buffer
		char* tmp = 0;
		stbds_arrinline(tmp, char, 4096);
		for (i = 0; i < arrlen(md->expansion) - 1; ++i)
		{
			int an;
			size_t off;

			off = arraddnindex(tmp, md->expansion[i].text_length);
			// copy macro-expansion text
			memcpy(tmp + off, md->expansion[i].text, md->expansion[i].text_length);

			an = md->expansion[i].argument_index;

			// sanity check added to satisfy static analysis; this shouldn't be possible
			if (arguments == NULL || an >= arrlen(arguments))
			{
				do_error(cs, "Internal error; arguments array not constructed as expected");
				return;
			}

			if (md->expansion[i].eliminate_comma_before_empty_arg)
			{
				// this flag is normally only set if both of the following are true:
				//    1. the previous expansion ends with a comma
				//    2. this argument is __VA_ARGS__
				// however, you could use it with non-__VA_ARGS__ and it will still work,
				// if you wanted to make some new extension

				if (arguments[an] == NULL)
				{
					// find the generated comma and set it to whitespace
					char* t = tmp + arrlen(tmp);
					while (*--t != ',')
						;
					*t = ' ';
				}
			}

			// copy argument substitution

			if (md->expansion[i].stringify_argument || md->expansion[i].concat_argument)
			{
				if (md->expansion[i].stringify_argument)
				{
					arrput(tmp, ' ');
					arrput(tmp, '"');
					// @TODO: stringify needs to insert backslashes before \ and "
					off = arraddnindex(tmp, arrlen(arguments[an]));
					memcpy(tmp + off, arguments[an], arrlen(arguments[an]));
					arrput(tmp, '"');
				}
				else
				{
					off = arraddnindex(tmp, arrlen(arguments[an]));
					memcpy(tmp + off, arguments[an], arrlen(arguments[an]));
				}
			}
			else
			{
				struct macro_definition* ffm = 0;
				acs.src = arguments[an];
				acs.src_length = arrlen(arguments[an]);
				acs.dest = tmp;
				acs.src_offset = 0;
				preprocess_string(&acs, in_macro_expansion, identifier, &ffm);
				tmp = acs.dest;
				if (ffm)
				{
					// concatenate it into the result string... if it's at the end, we'll catch it again in the processing below
					size_t len = strlen(ffm->symbol_name);
					off = arraddnindex(tmp, len);
					memcpy(tmp + off, ffm->symbol_name, len);
				}
				if (cs->context->stop)
					return;
			}
		}

		{
			size_t off = arraddnindex(tmp, md->expansion[i].text_length);
			memcpy(tmp + off, md->expansion[i].text, md->expansion[i].text_length);
		}

		arrput(tmp, 0);
#if SSE_READ_PADDING
		arrsetcap(tmp, arrlen(tmp) + SSE_READ_PADDING);
#endif

		{
			for (i = 0; i < arrlen(arguments); ++i)
				arrfree(arguments[i]);
			arrfree(arguments);
		}

		ncs.src = tmp;
		ncs.src_length = arrlen(tmp);
		ncs.src_offset = 0;

		//////////////////////////////////////////////////////////////////
		//
		//   5. preprocess temporary buffer
		//

		md->disabled = 1;  // don't disable earlier, because if you CALL foo() WITH foo, it should still get expanded
		preprocess_string(&ncs, IN_MACRO_yes, identifier, final_function_macro);
		cs->dest = ncs.dest;
		arrfree(ncs.src);

		md->disabled = 0;
	}
}

//////////////////////////////////////////////////////////////////////////////
//
//  DIRECTIVE PROCESSING
//

// it's already been cleaned of backslash-continuation and is NUL-terminated
static char* macro_expand_directive(parse_state* cs, const char* p, char* directive)
{
	parse_state ncs;
	ncs = *cs;
	ncs.src = p;
	ncs.src_offset = 0;
	ncs.parent = cs;
	ncs.dest = 0;
	preprocess_string(&ncs, IN_MACRO_directive, directive, NULL);
	return ncs.dest;
}

static int evaluate_if(parse_state* cs, const char* p, int* syntax_error)
{
	char* dest = 0;
	stbds_arrinline(dest, char, 1024);

	int result = 0;
	parse_state ncs;
	ncs = *cs;
	ncs.src = p;
	ncs.src_length = strlen(p);
	ncs.src_offset = 0;
	ncs.parent = cs;
	ncs.dest = dest;

	preprocess_string(&ncs, IN_MACRO_if_condition, "#if", NULL);

	// recursive descent evaluator
	if (!cs->context->stop && ncs.dest != NULL)
	{
		arrput(ncs.dest, '\0');
		result = evaluate_integer_constant_expression_as_condition(ncs.dest, syntax_error);
	}
	else
	{
		*syntax_error = 0;
	}
	arrfree(ncs.dest);
	return result;
}

static const char* parse_pp_identifier_in_directive(parse_state* ps, const char* text, char* identifier, size_t identifier_buffer_length, char* directive, size_t* idlen)
{
	size_t i;
	const char* p = preprocessor_skip_whitespace_simple(text);
	*idlen = 0;
	identifier[0] = 0;
	if (!char_is_pp_identifier_first(*p))
	{
		do_error_char_code(ps, PP_RESULT_invalid_char, "Invalid character %s after '#%s'", *p, directive);
		return p;
	}
	identifier[0] = *p++;
	for (i = 1; i < identifier_buffer_length - 1; ++i)
	{
		if (!char_is_pp_identifier(*p))
			break;
		identifier[i] = *p++;
	}
	*idlen = i;
	identifier[i] = 0;
	if (char_is_pp_identifier(*p))
	{
		do_error_code(ps, PP_RESULT_identifier_too_long, "'#%s' identifier too long", directive);
	}
	return p;
}


enum
{
	RETURN_BEHAVIOR_return,
	RETURN_BEHAVIOR_break,
	RETURN_BEHAVIOR_skip_to_end_of_line,
};

static char* preprocess_string_from_file(parse_state* ps, const char* filename, const char* text, size_t textlen, int conditional_compilation_nesting, int is_header);

static void pragma_once(pp_context* c, const char* filename, char* macroname)
{
	if (macroname)
		macroname = stb_arena_alloc_string(&c->macro_arena, macroname);
	else
		macroname = (char*)(size_t)1;
	shput(c->once_map, filename, macroname);
}


static int process_include(parse_state* cs, const char* start, conditional_state* cons)
{
	int ret = RETURN_BEHAVIOR_break;
	pp_context* c = cs->context;
	const char *filename, *p;
	p = preprocessor_skip_whitespace_simple(start);
	// @TODO: macro expansion
	if (!(*p == '<' || *p == '"'))
	{
		do_error_char(cs, "Expected filename after '#include', found character %s", *p);
		return RETURN_BEHAVIOR_return;
	}
	else
	{
		char* once;
		char path[MAX_FILENAME];
		const char *start_char = p + 1, *old_p;
		char end_char = (*p == '<') ? '>' : '"';
		++p;
		while (*p != end_char)
		{
			if (*p == end_char)
				break;
			if (char_is_end_of_line(*p))
			{
				char cbuf[2] = {0};
				cbuf[0] = *p;
				do_error(cs, "Missing closing '%s' in '#include", cbuf);
				return RETURN_BEHAVIOR_return;
			}
			++p;
		}

		filename = (*resolveinclude_callback)(start_char, p - start_char, cs->filename, c->custom_context);
		if (filename == NULL)
		{
			memcpy(path, start_char, p - start_char);
			path[p - start_char] = 0;
			do_error(cs, "File '%s' not found", path);
			return RETURN_BEHAVIOR_return;
		}

		old_p = p;
		p = preprocessor_skip_whitespace_simple(p + 1);
		if (*p != 0)
		{
			do_error_char(cs, "Unexpected character %s after filename in '#include'", *p);
			return RETURN_BEHAVIOR_return;
		}

		once = shget(c->once_map, filename);
		if (once == (char*)(size_t)1)
		{
			// #pragma once, so skip
			c->num_onced_files += 1;
		}
		else
		{
			struct macro_definition* md = NULL;

			if (once != NULL)
				md = (struct macro_definition*)stringhash_get(&c->macro_map, once, strlen(once));

			// if macro is defined, it's an include guard, which means if it's defined we can skip processing
			if (md == NULL)
			{
				const char* data;
				char *result;
				size_t len;
				data = (*loadfile_callback)(filename, c->custom_context, &len);

				if (data == NULL)
				{
					fprintf(stderr, "Preprocessor fatal internal error: was able to open '%s' once, but not the second time.\n", filename);
					exit(1);
				}

				c->num_includes += 1;
				result = preprocess_string_from_file(cs, filename, data, len, cons->conditional_compilation_nesting_level, 1);
				
				(*freefile_callback)(filename, data, c->custom_context);

				cs->dest = result;
				if (c->stop)
					return RETURN_BEHAVIOR_return;
			}
			else
			{
				c->num_onced_files += 1;
			}
		}
	}
	return ret;
}

static void macro_free(struct macro_definition* md)
{
	if (md->num_parameters >= 0)
	{
		arrfree(md->expansion);
	}
}

// process # directives
static void process_directive(parse_state* cs, conditional_state* cons)
{
	int n;
	pp_context* c = cs->context;
	const char *p, *endptr;
	const char* alloc = NULL;	 // all exit paths SHOULD do 'if (alloc) arrfree(alloc)', but we allow it to leak on error-handling returns
	char buffer[2048 + SSE_READ_PADDING];
	int dummy;	// uninitialized, should never get incremented by parse_directive_after_hash

	STB_COMMON_ASSERT(cs->src[cs->src_offset] == '#');
	p = copy_line_without_comments(buffer, sizeof(buffer) - SSE_READ_PADDING, cs->src + cs->src_offset + 1, &cs->src_line_number, &endptr);
	if (p != buffer)
		alloc = p;

	if (endptr == NULL)
	{
		do_error(cs, "Unclosed '/*' comment contained in directive");
		return;
	}

	cs->src_offset = endptr - cs->src;
	// the above leaves src_offset pointing to the newline at the end of the directive

	n = parse_directive_after_hash(&p, &dummy);

	// when we reach here, n is the directive keyword, or if it's
	// a value that's not defined as a directive, then it could be
	// anything

	switch (n & 31)
	{  // mask to try to avoid range-checked switch
		// not a known directive:
		case HASH_none:
		// random hash values which mostly don't match but could on garbage;
		// included here so every value is specified to encourage a full jump table
		case 1:
		case 2:
		case 3:
		case 5:
		case 7:
		case 13:
		case 14:
		case 16:
		case 17:
		case 19:
		case 20:
		case 21:
		case 23:
		case 25:
		case 26:
		case 27:
		case 28:
		case 29:
		{
			// p points to first newline or non-whitespace character after '#'
			switch (*p)
			{
				case '\r':
				case '\n':
					// empty directive; just skip it, let regular code copy the newline
					break;

				default:
					// invalid directive, produce a diagnostic
					if (!char_is_pp_identifier_first(*p))
					{
						do_error_char_code(cs, PP_RESULT_invalid_char, "Invalid character %s after '#'", *p);
					}
					else
					{
						// if no warning, skip parsing
						if (pp_result_mode[PP_RESULT_unrecognized_directive] != PP_RESULT_MODE_no_warning)
						{
							static char* directives[] = {
								"define", "else", "elif", "endif", "error", "if", "ifdef", "ifndef", "include", "line", "pragma", "undef", "warning",
							};
							char identifier[64] = { 0 }, *match = 0;
							int i;
							for (i = 0; i < sizeof(identifier); ++i)
							{
								if (!char_is_pp_identifier(p[i]))
									break;
								identifier[i] = p[i];
							}
							if (i < sizeof(identifier) && pp_result_mode[PP_RESULT_unrecognized_directive] <= did_you_mean_threshold)
							{
								match = find_closest_matching_string(
									identifier, directives, sizeof(directives) / sizeof(directives[0]),
									i > 7 ? 2 + i - 7 : 2);	 // if the string is long, allow more errors, e.g. they put junk on the end
							}
							if (match)
								do_error_code(cs, PP_RESULT_unrecognized_directive, "Unrecognized preprocessor directive '#%s', did you mean '%s'?", identifier,
											  match);
							else
								do_error_code(cs, PP_RESULT_unrecognized_directive, "Unrecognized preprocessor directive '#%s'", identifier);
						}
					}
					if (alloc)
						arrfree(alloc);
					return;
			}
			break;
		}

		case HASH_line:
		{
			// parse line number and file name and apply these to the parser state. this will trigger a line directive to be emitted with the correct format.

			p = preprocessor_skip_whitespace_simple(p);
			if (strlen(p) == 0)
				return;

			char* number_end = NULL;
			long long int line_number = strtoll(p, &number_end, 10);
			if (!number_end || number_end == p || llabs(line_number) >= INT_MAX)
			{
				do_error_code(cs, PP_RESULT_invalid_line_linenumber, "Invalid #line line number");
				return;
			}

			cs->src_line_number = (int)line_number;

			p = preprocessor_skip_whitespace_simple(number_end);

			// file path next. optional - we're done if nothing provided.
			if (char_is_end_of_line(*p))
				return;

			// file path should be surrounded by quotes.
			if (!char_is_quote(*p++))
			{
				do_error_code(cs, PP_RESULT_malformed_line_filename, "Malformed #line file name");
				return;
			}

			// find end of file path
			const char* path_end = p;
			while (*path_end != '"' && !char_is_end_of_line(*path_end))
				path_end++;

			if (*path_end != '"')
			{
				do_error_code(cs, PP_RESULT_malformed_line_filename, "Malformed #line file name");
				return;
			}

			ptrdiff_t len = path_end - p;
			if (len <= 0)
			{
				do_error_code(cs, PP_RESULT_malformed_line_filename, "Malformed #line file name");
				return;
			}

			// apply new file path
			cs->filename = stb_arena_alloc_string_length(&c->macro_arena, p, len);

			return;
		}

		case HASH_pragma:
		{
			char identifier[256];
			size_t idlen;

			p = preprocessor_skip_whitespace_simple(p);
			(void)parse_pp_identifier_in_directive(cs, p, identifier, sizeof(identifier), "pragma", &idlen);
			// check for preprocessor tokens we recognize
			if (idlen >= 3) // don't bother comparing if the identifier is < 3 chars long (to satisfy some CA failure conditions)
			{
				switch (identifier[0])
				{
				case 'o':
					if (0 == strcmp(identifier, "once"))
					{
						pragma_once(cs->context, cs->filename, NULL);
						goto handled;
					}
					break;
				case 'p':
					if (0 == strcmp(identifier, "push") || 0 == strcmp(identifier, "push_macro"))
					{
						// @TODO implement pragma push_macro
						goto handled;
					}
					if (0 == strcmp(identifier, "pop") || 0 == strcmp(identifier, "pop_macro"))
					{
						// @TODO implement pragma pop_macro
						goto handled;
					}
					if (0 == strcmp(identifier, "pack"))
					{
						// do macro expansion to handle #pragma pack(push,_CRT_PACKING) in vadefs.h
						// do_expansion:
						{
							// @TODO: this is macro-substituting "pack" as well
							char* temp = macro_expand_directive(cs, p, "#pragma pack");
							// replace the unexpanded directive text with the expanded directive text
							if (alloc)
								arrfree(alloc);
							p = alloc = temp;
						}
					}
					break;
				case 'm':
					if (0 == strcmp(identifier, "message"))
					{
						const char* message_end;
						p = scan_pp_identifier(p);
						if (*p++ != '(')
						{
							do_error_code(cs, PP_RESULT_malformed_pragma_message, "Malformed #pragma message");
							goto handled;
						}
						char quote = *p++;

						if (!char_is_quote(quote))
						{
							do_error_code(cs, PP_RESULT_malformed_pragma_message, "Malformed #pragma message");
							goto handled;
						}

						message_end = p;
						while (*message_end != quote && !(char_is_end_of_line(*message_end)))
							message_end++;

						if (!char_is_quote(*message_end))
						{
							do_error_code(cs, PP_RESULT_malformed_pragma_message, "Malformed #pragma message");
							goto handled;
						}

						ptrdiff_t len = (message_end - p) + 1;

						pp_diagnostic diag;
						diag.error_level = PP_RESULT_MODE_no_warning;
						diag.diagnostic_code = PP_RESULT_ok;
						diag.message = (char*)STB_COMMON_MALLOC(len);
						memcpy(diag.message, p, len - 1);
						diag.message[len - 1] = 0;
						diag.where = 0;

						arrpush(cs->context->diagnostics, diag);
						goto handled;
					}
				}
			}

			// copy pragma through to output... existing code will output the newline, so don't do it here
			{
				size_t out_length = strlen(p);
				size_t s = arraddnindex(cs->dest, 8);
				memcpy(cs->dest + s, "#pragma ", 8);
				s = arraddnindex(cs->dest, out_length);
				memcpy(cs->dest + s, p, out_length);
			}

		handled:
			if (alloc)
				arrfree(alloc);
			return;
		}

		case HASH_define:
		{
			struct macro_definition* old;
			struct macro_definition* m;

			m = create_macro_definition(cs, p);
			if (c->include_guard_detect)
				c->last_macro_definition = m;

			if (m != NULL)
			{
				old = (struct macro_definition*)stringhash_get(&c->macro_map, m->symbol_name, m->symbol_name_length);
				if (old != NULL)
				{
					// @TODO: compare new definition to old and produce error/warning
					macro_free(old);
					stringhash_delete(&c->macro_map, m->symbol_name, m->symbol_name_length);
				}
				stringhash_put(&c->macro_map, m->symbol_name, m->symbol_name_length, m);
				update_macro_filter(c, m->symbol_name, m->symbol_name_length);
			}
			if (alloc)
				arrfree(alloc);
			return;
		}

		case HASH_undef:
		{
			char identifier[1025];
			size_t idlen;
			struct macro_definition* md;

			// process #undef

			p = parse_pp_identifier_in_directive(cs, p, identifier, sizeof(identifier), "undef", &idlen);
			if (identifier[0] == 0)
				if (alloc)
					arrfree(alloc);

			md = (struct macro_definition*)stringhash_get(&c->macro_map, identifier, idlen);

			if (md != NULL)
			{
				if (md->predefined)
				{
					if (do_error_code(cs, PP_RESULT_undef_of_predefined_macro, "Cannot #undef predefined macro '%s'", identifier))
						return;
				}
				else
				{
					macro_free(md);
					stringhash_delete(&c->macro_map, identifier, idlen);
					shput(c->undef_map, identifier, (struct macro_definition*)(size_t)1);
				}
			}
			else
			{
				// VS header workaround
				if (0 == strcmp(identifier, "long"))
					return;

				// process #undef error
				if (pp_result_mode[PP_RESULT_undef_of_undefined_macro] != PP_RESULT_MODE_no_warning)
				{
					if (shget(c->undef_map, identifier))
					{
						// @DIAGNOSTIC: record the file/line of the last #undef, so we can report it here?
						// however, recording #file requires making sure it stays alive, but we don't want to do it multiple times?
						// then again #undefs are pretty rare, so maybe we can just arena alloc it on undef
						if (do_error_code(cs, PP_RESULT_undef_of_undefined_macro,
										  "Tried to #undef '%s', which is not a defined macro name. However, it was defined previously.", identifier))
							return;
					}
					else if (pp_result_mode[PP_RESULT_undef_of_undefined_macro] <= did_you_mean_threshold)
					{
						char* best = 0;
						int dist = 9999, i;
						int min_dist = n > 5 ? 3 : n > 3 ? 2 : 1;
						n = (int)idlen;
						if (n > 1)
						{
							find_closest_match_init(&dist, &best, min_dist);
							if (n >= 4)
							{
								// in case it's a warning, we don't want to be too slow, so accelerate if we can
								for (i = 0; i < arrlen(c->macro_map.pair); ++i)
								{
									// quick cull... if the first four characters mismatch, it's not a candidate, since we only allow max 3 errors
									char* s = c->macro_map.pair[i].key;
									if (s)
									{
										if (s[0] && s[1] && s[2] && s[3] && s[0] != identifier[0] && s[1] != identifier[1] && s[2] != identifier[2] &&
											s[3] != identifier[3])
											;
										else
											find_closest_match_step(identifier, s, &dist, &best);
									}
								}
							}
							else
							{
								for (i = 0; i < arrlen(c->macro_map.pair); ++i)
									if (c->macro_map.pair[i].key)
										find_closest_match_step(identifier, c->macro_map.pair[i].key, &dist, &best);
							}
						}
						if (dist <= min_dist
								? do_error_code(cs, PP_RESULT_undef_of_undefined_macro,
												"Tried to #undef '%s', which is not a defined macro name. Did you mean '%s'?", identifier, best)
								: do_error_code(cs, PP_RESULT_undef_of_undefined_macro, "Tried to #undef '%s', which is not a defined macro name.", identifier))
							return;
					}
					else
					{
						if (do_error_code(cs, PP_RESULT_undef_of_undefined_macro, "Tried to #undef '%s', which is not a defined macro name.", identifier))
							return;
					}
				}
			}
			if (alloc)
				arrfree(alloc);
			return;
		}

		case HASH_include:
		{
			process_include(cs, p, cons);
			if (alloc)
				arrfree(alloc);
			return;
		}

			//////////////////////////////////////////////////////////////////////////////
			//
			//  CONDITIONAL PROCESSING
			//

		case HASH_ifdef:
		{
			ifdef_info ifinfo;
			struct macro_definition* m;
			char identifier[1025];
			size_t idlen;

			ifinfo.line = cs->src_line_number;
			ifinfo.type = HASH_ifdef;
			p = parse_pp_identifier_in_directive(cs, p, identifier, sizeof(identifier), "ifdef", &idlen);
			if (c->stop)
				return;

			arrpush(c->ifdef_stack, ifinfo);
			++cons->conditional_compilation_nesting_level;

			m = (struct macro_definition*)stringhash_get(&c->macro_map, identifier, idlen);
			if (m == NULL || identifier[0] == 0)
				cons->conditionally_disable = CONDITIONAL_skip_current_block;
			else
				cons->conditionally_disable = CONDITIONAL_none;

			p = preprocessor_skip_whitespace_simple(p);
			if (*p != 0)
			{
				do_error_char_code(cs, PP_RESULT_unexpected_character_after_end_of_directive, "Expected end of line in #ifdef, but found character %s", *p);
			}
			break;
		}

		case HASH_ifndef:
		{
			ifdef_info ifinfo;
			struct macro_definition* m;
			char identifier[1025];
			size_t idlen;

			ifinfo.line = cs->src_line_number;
			ifinfo.type = HASH_ifndef;
			p = parse_pp_identifier_in_directive(cs, p, identifier, sizeof(identifier), "ifndef", &idlen);
			if (c->stop)
				return;

			if (c->include_guard_detect)
				c->last_ifndef = stb_arena_alloc_string(&c->macro_arena, identifier);

			arrpush(c->ifdef_stack, ifinfo);
			++cons->conditional_compilation_nesting_level;

			m = (struct macro_definition*)stringhash_get(&c->macro_map, identifier, idlen);
			if (m != NULL)
				cons->conditionally_disable = CONDITIONAL_skip_current_block;
			else
				cons->conditionally_disable = CONDITIONAL_none;

			p = preprocessor_skip_whitespace_simple(p);
			if (*p != 0)
			{
				do_error_char_code(cs, PP_RESULT_unexpected_character_after_end_of_directive, "Expected end of line in #ifdef, but found character %s.", *p);
			}
			break;
		}

		case HASH_if:
		{
			ifdef_info ifinfo;
			int result, result_code;

			ifinfo.line = cs->src_line_number;
			ifinfo.type = HASH_if;

			p = preprocessor_skip_whitespace_simple(p);
			result = evaluate_if(cs, p, &result_code);

			if (result_code != CE_RESULT_ok)
			{
				char* err_msg;
				int ce_result;
				switch (result_code)
				{
				case CE_RESULT_division_by_zero:
					err_msg = "Division by zero occurred when evaluating conditional expression";
					ce_result = PP_RESULT_conditional_expression_division_by_zero;
					break;
				case CE_RESULT_overflow:
					err_msg = "Overflow occurred when evaluating conditional expression";
					ce_result = PP_RESULT_conditional_expression_overflow;
					break;
				case CE_RESULT_syntax_error:
					err_msg = "Syntax error occurred when evaluating conditional expression";
					ce_result = PP_RESULT_syntax_error_conditional_expression;
					break;
				}
				
				do_error_explicit(cs, ce_result, cs->src_line_number, err_msg);
				return;
			}

			arrpush(c->ifdef_stack, ifinfo);
			++cons->conditional_compilation_nesting_level;

			if (result)
				cons->conditionally_disable = CONDITIONAL_none;
			else
				cons->conditionally_disable = CONDITIONAL_skip_current_block;
			break;
		}

		case HASH_elif:
		handle_elif_after_disabled_block:
		{
			ifdef_info ifinfo;
			int result, syntax_error;

			ifinfo.line = cs->src_line_number;
			ifinfo.type = HASH_elif;

			if (cons->conditional_compilation_nesting_level == 0)
			{
				do_error(cs, "Found '#elif' without a prior '#if', '#ifdef', or '#ifndef' in the same file.");
				return;
			}

			if (arrlast(c->ifdef_stack).type == HASH_else)
			{
				do_error(cs, "Found '#elif' after previous '#else'");
				error_supplement(cs, PP_RESULT_ERROR, arrlast(c->ifdef_stack).line, "Location of previous '#else'");
				return;
			}

			if (cons->conditionally_disable == CONDITIONAL_none)
			{
				// previous block was enabled, so we're done
				cons->conditionally_disable = CONDITIONAL_skip_all_blocks;
				break;
			}

			p = preprocessor_skip_whitespace_simple(p);
			result = evaluate_if(cs, p, &syntax_error);
			if (syntax_error)
			{
				do_error(cs, "Syntax error in conditional expression of '#elif'");
				return;
			}

			arrlast(c->ifdef_stack) = ifinfo;

			if (result)
				cons->conditionally_disable = CONDITIONAL_none;
			else
				cons->conditionally_disable = CONDITIONAL_skip_current_block;
			break;
		}

		case HASH_else:
		handle_else_after_disabled_block:
		{
			ifdef_info ifinfo;

			p = preprocessor_skip_whitespace_simple(p);
			ifinfo.type = HASH_else;
			ifinfo.line = cs->src_line_number;

			if (*p != 0)
			{
				if (do_error_char_code(cs, PP_RESULT_unexpected_character_after_end_of_directive, "Expected end of line in #else, but found character %s.", *p))
				{
					return;
				}
			}

			if (cons->conditional_compilation_nesting_level == 0)
			{
				do_error(cs, "Found '#else' without a prior '#if', '#ifdef', or '#ifndef' in the same file.");
				return;
			}

			if (arrlast(c->ifdef_stack).type == HASH_else)
			{
				// can't have two else clauses
				do_error(cs, "Found '#else' after previous '#else'");
				error_supplement(cs, PP_RESULT_ERROR, arrlast(c->ifdef_stack).line, "Location of previous '#else'");
				return;
			}

			if (cons->conditionally_disable == CONDITIONAL_none)
			{
				// previous block was enabled, so we're done
				cons->conditionally_disable = CONDITIONAL_skip_all_blocks;
			}
			else
			{
				// previous block was disabled, so enable
				cons->conditionally_disable = CONDITIONAL_none;
			}

			// replace old entry with new entry
			arrlast(c->ifdef_stack) = ifinfo;
			break;
		}

		handle_endif_after_disabled_block:
		case HASH_endif:
		{
			p = preprocessor_skip_whitespace_simple(p);
			if (*p != 0)
			{
				if (do_error_char_code(cs, PP_RESULT_unexpected_character_after_end_of_directive, "Expected end of line in #endif, but found character %s.", *p))
				{
					return;
				}
			}

			if (cons->conditional_compilation_nesting_level == 0)
			{
				do_error(cs, "Found '#endif' without a prior '#if', '#ifdef', or '#ifndef' in the same file.");
				return;
			}
			(void)arrpop(c->ifdef_stack);
			--cons->conditional_compilation_nesting_level;
			cons->conditionally_disable = CONDITIONAL_none;

			if (cs->include_guard_candidate_macro_name != 0 && cons->conditional_compilation_nesting_level == cs->include_guard_endif_level)
			{
				// found the #endif for a leading #ifndef/#define pair, so now check if there's
				// any code after the #endif
				int state;

				state = scan_whitespace_and_comments(cs);

				if (state == PP_WHITESCAN_STATE_stop_eof)
				{
					// only whitespace and comments after #endif from include guard, so it's a real include guard!
					// printf("Detected include guard: %s (%s)\n", cs->include_guard_candidate_macro_name, cs->filename);
					pragma_once(cs->context, cs->filename, cs->include_guard_candidate_macro_name);
				}
				else
				{
					// printf("Invalid include guard: %s (%s)\n", cs->include_guard_candidate_macro_name, cs->filename);
					rewind_to_start_of_line(cs);
				}
				cs->include_guard_candidate_macro_name = 0;
				return;
			}
			break;
		}

		case HASH_error:
		case HASH_warning:
			p = preprocessor_skip_whitespace_simple(p);
			const char* str_end = p;
			while (*str_end != 0 && !(char_is_end_of_line(*str_end)))
				str_end++;
		
			ptrdiff_t len = (str_end - p) + 1;
			char* str = (char*)STB_COMMON_MALLOC(len);
			memcpy(str, p, len - 1);
			str[len - 1] = 0;

			error_explicit(cs, ((n & 31) == HASH_error) ? PP_RESULT_ERROR : PP_RESULT_explicit_warning, 0, str);
	}

	// each case above individually scans to end of line so they can
	// report the error unambiguously, so we should definitely be at
	// the end of the line at this point

	if (alloc)
		arrfree(alloc);

	if (cons->conditionally_disable == CONDITIONAL_none)
		return;
	else
	{
		int disable_nesting_level = 0;
		int newlines = 0;

		alloc = 0;

		p = cs->src + cs->src_offset;

		for (;;)
		{
			const char* q;
			const char* hash_location;
			p = scan_to_directive(p, &newlines);  // fast scanner that skips non-directive content

			if (*p == 0)
			{
				cs->src_line_number += newlines;
				cs->num_disabled_lines += newlines;
				{
					int i;
					for (i = (int)arrlen(c->ifdef_stack) - 1; i >= cs->conditional_nesting_depth_at_start; --i)
					{
						do_error(cs, "No #endif found for previous #%s", &directive_hash[c->ifdef_stack[i].type].name[0]);
						error_supplement(cs, PP_RESULT_ERROR, c->ifdef_stack[i].line, "Location of #%s", &directive_hash[c->ifdef_stack[i].type].name[0]);
					}
				}
				return;
			}

			STB_COMMON_ASSERT(*p == '#');
			hash_location = p;
			q = p + 1;
			cs->src_line_number += newlines;
			c->num_disabled_lines += newlines;
			n = parse_directive_after_hash(&q, &cs->src_line_number);
			p = q;
			newlines = 0;

			switch (n)
			{
				case HASH_if:
				case HASH_ifdef:
				case HASH_ifndef:
				{
					ifdef_info ifinfo;

					ifinfo.type = n;
					ifinfo.line = cs->src_line_number;
					++disable_nesting_level;
					arrpush(c->ifdef_stack, ifinfo);
					break;
				}
				case HASH_else:
					if (disable_nesting_level == 0)
					{
						if (cons->conditionally_disable == CONDITIONAL_skip_current_block)
						{
							p = copy_line_without_comments(buffer, sizeof(buffer) - SSE_READ_PADDING, p, &cs->src_line_number, &endptr);
							if (p != buffer)
								alloc = p;
							if (endptr == NULL)
							{
								do_error(cs, "Unclosed '/*' comment contained in #else");
								return;
							}
							cs->src_offset = endptr - cs->src;
							goto handle_else_after_disabled_block;
						}
						if (arrlast(c->ifdef_stack).type == HASH_else)
						{
							// we were in an else block, and now there's another else block, so produce an error
							do_error(cs, "Multiple #else blocks before #endif.");
							return;
						}
					}
					arrlast(c->ifdef_stack).type = n;
					arrlast(c->ifdef_stack).line = cs->src_line_number;
					break;

				case HASH_endif:
					if (disable_nesting_level == 0)
					{
						p = copy_line_without_comments(buffer, sizeof(buffer) - SSE_READ_PADDING, p, &cs->src_line_number, &endptr);
						if (p != buffer)
							alloc = p;
						if (endptr == NULL)
						{
							do_error(cs, "Unclosed '/*' comment contained in #endif");
							return;
						}
						cs->src_offset = endptr - cs->src;
						goto handle_endif_after_disabled_block;
					}
					--disable_nesting_level;
					arrpop(c->ifdef_stack);
					break;

				case HASH_elif:
					if (disable_nesting_level == 0)
					{
						if (cons->conditionally_disable == CONDITIONAL_skip_current_block)
						{
							p = copy_line_without_comments(buffer, sizeof(buffer) - SSE_READ_PADDING, p, &cs->src_line_number, &endptr);
							if (p != buffer)
								alloc = p;
							if (endptr == NULL)
							{
								do_error(cs, "Unclosed '/*' comment contained in #elif");
								return;
							}
							cs->src_offset = endptr - cs->src;
							goto handle_elif_after_disabled_block;
						}
					}
					arrlast(c->ifdef_stack).type = HASH_elif;
					arrlast(c->ifdef_stack).line = cs->src_line_number;
					break;

				default:
					break;
			}

			for (;;)
			{
				while (!char_is_end_of_line(*p))
					++p;
				if (*p == 0)  // if end of file, stop
					break;
				if (p[-1] != '\\')	// if not preceded by backslash, stop
					break;
				++newlines;
				p += newline_char_count(p[0], p[1]);
			}
		}
		/* NOTREACHED */  // end of infinite loop
	}
}

int preprocessor_automatic_include_guard_detection = 1;
static char* preprocess_string_from_file(parse_state* ps, const char* filename, const char* text, size_t textlen, int conditional_compilation_nesting, int is_header)
{
	STB_COMMON_ASSERTF(text != NULL,
		"loader returned null source buffer for include (filename=%s, text=%p, textlen=%zu)",
		filename ? filename : "<null>", (const void*)text, textlen);

	parse_state cs = *ps;
	cs.parent = ps;
	cs.filename = filename;
	cs.last_output_filename = "";  // force #line directive to include filename
	cs.src = text;
	cs.copied_identifier_length = 0;
	cs.src_line_number = 1;
	cs.dest_line_number = 1;
	cs.in_leading_whitespace = 1;
	cs.src_length = textlen;
	cs.src_offset = 0;
	cs.conditional_nesting_depth_at_start = conditional_compilation_nesting;
	// dest is inherited
	preprocess_string(&cs, is_header && preprocessor_automatic_include_guard_detection ? IN_MACRO_include_guard_scan : IN_MACRO_no, 0, 0);
	cs.context->num_lines += cs.src_line_number;
	ps->last_output_filename = "";
	return cs.dest;
}

#define DEFAULT -1
#define ANY_LETTER -2
#define ANY_DIGIT PP_CHAR_CLASS_digit

#define STATE_ANY_WHITESPACE -20
#define STATE_ANY_MULTILINE_COMMENT -21
#define STATE_DEFAULT -22

static void transition(int a, int b, int c)
{
	if (a == STATE_ANY_WHITESPACE)
	{
		transition(PP_STATE_ready, b, c);
		transition(PP_STATE_in_leading_whitespace, b, c);
		transition(PP_STATE_saw_cr, b, c);
		transition(PP_STATE_saw_lf, b, c);
	}
	else if (a == STATE_ANY_MULTILINE_COMMENT)
	{
		transition(PP_STATE_comment_multiline, b, c);
		transition(PP_STATE_comment_multiline_seen_star, b, c);
		transition(PP_STATE_comment_multiline_seen_cr, b, c);
		transition(PP_STATE_comment_multiline_seen_lf, b, c);
	}
	else if (a == STATE_DEFAULT)
	{
		int i;
		for (i = 0; i < PP_STATE_active_count; ++i)
			transition(i, b, c);
	}
	else
	{
		if (b == DEFAULT)
			memset(pp_transition_table[a], c, PP_CHAR_CLASS_count);
		else if (b == ANY_LETTER)
		{
			transition(a, PP_CHAR_CLASS_idtype, c);
			transition(a, PP_CHAR_CLASS_idtype_e_E, c);
		}
		else
			pp_transition_table[a][b] = (uint8)c;
	}
}

static void initscan_transition(int a, int b, int c)
{
	if (a == STATE_ANY_MULTILINE_COMMENT)
	{
		initscan_transition(PP_WHITESCAN_STATE_comment_multiline, b, c);
		initscan_transition(PP_WHITESCAN_STATE_comment_multiline_seen_star, b, c);
		initscan_transition(PP_WHITESCAN_STATE_comment_multiline_seen_cr, b, c);
		initscan_transition(PP_WHITESCAN_STATE_comment_multiline_seen_lf, b, c);
	}
	else if (a == STATE_DEFAULT)
	{
		int i;
		for (i = 0; i < PP_WHITESCAN_STATE_active_count; ++i)
			initscan_transition(i, b, c);
	}
	else
	{
		if (b == DEFAULT)
			memset(pp_transition_initscan[a], c, PP_CHAR_CLASS_count);
		else if (b == ANY_LETTER)
		{
			initscan_transition(a, PP_CHAR_CLASS_idtype, c);
			initscan_transition(a, PP_CHAR_CLASS_idtype_e_E, c);
		}
		else
			pp_transition_initscan[a][b] = (uint8)c;
	}
}

void init_preprocessor_scanner(void)
{
	int i;

	// build character class tables

	STB_COMMON_ASSERT(PP_CHAR_CLASS_count <= sizeof(pp_transition_table[0]));

	memset(pp_char_class, PP_CHAR_CLASS_default, 256);

	pp_char_class[0] = PP_CHAR_CLASS_eof;

	for (i = 0; i < 127; ++i)
	{
		if (isalpha(i))
			pp_char_class[i] = PP_CHAR_CLASS_idtype;
		if (isdigit(i))
			pp_char_class[i] = PP_CHAR_CLASS_digit;
		if (isspace(i))
			pp_char_class[i] = PP_CHAR_CLASS_whitespace;
	}
	STB_COMMON_ASSERT(pp_char_class['\t'] == PP_CHAR_CLASS_whitespace);

	pp_char_class['e'] = pp_char_class['E'] = PP_CHAR_CLASS_idtype_e_E,

	pp_char_class['\r'] = PP_CHAR_CLASS_cr;
	pp_char_class['\n'] = PP_CHAR_CLASS_lf;

	pp_char_class['$'] = PP_CHAR_CLASS_idtype;	// common C extension: allow $ in identifiers
	pp_char_class['_'] = PP_CHAR_CLASS_idtype;

	pp_char_class['+'] = PP_CHAR_CLASS_plusminus;
	pp_char_class['-'] = PP_CHAR_CLASS_plusminus;

	pp_char_class['/'] = PP_CHAR_CLASS_slash;
	pp_char_class['*'] = PP_CHAR_CLASS_star;
	pp_char_class['.'] = PP_CHAR_CLASS_period;

	pp_char_class['\"'] = PP_CHAR_CLASS_quotation;
	pp_char_class['\''] = PP_CHAR_CLASS_apostrophe;
	pp_char_class['\\'] = PP_CHAR_CLASS_backslash;

	pp_char_class['#'] = PP_CHAR_CLASS_hash;

	// the scanner does the same as above, but doesn't stop on identifiers
	memcpy(pp_scan_char_class, pp_char_class, 256);
	for (i = 0; i < 256; ++i)
		if (pp_scan_char_class[i] == PP_CHAR_CLASS_idtype || pp_scan_char_class[i] == PP_CHAR_CLASS_idtype_e_E)
			pp_scan_char_class[i] = PP_CHAR_CLASS_default;

	// properties of transitions (stored as properties of end state)
	memset(pp_state_is_end_of_line, 0, sizeof(pp_state_is_end_of_line));
	pp_state_is_end_of_line[PP_STATE_saw_cr] = 1;
	pp_state_is_end_of_line[PP_STATE_saw_lf] = 1;
	pp_state_is_end_of_line[PP_STATE_comment_multiline_seen_cr] = 1;
	pp_state_is_end_of_line[PP_STATE_comment_multiline_seen_lf] = 1;

	// state machine transitions
	// by default, all states point to eof so we catch missing transitions
	for (i = 0; i < PP_STATE_active_count; ++i)
		transition(i, DEFAULT, PP_STATE_eof);

	// Disable V557 due to false positive and the use of STATE_DEFAULT
	//-V:transition:557

	// every character class has a default behavior:
	transition(STATE_DEFAULT, PP_CHAR_CLASS_default, PP_STATE_ready);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_whitespace, PP_STATE_ready);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_cr, PP_STATE_saw_cr);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_lf, PP_STATE_saw_lf);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_period, PP_STATE_saw_period);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_digit, PP_STATE_in_number);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_idtype, PP_STATE_saw_identifier_start);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_idtype_e_E, PP_STATE_saw_identifier_start);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_apostrophe, PP_STATE_in_char_literal);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_quotation, PP_STATE_in_string_literal);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_backslash, PP_STATE_saw_backslash);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_slash, PP_STATE_saw_slash);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_star, PP_STATE_ready);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_plusminus, PP_STATE_ready);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_hash, PP_STATE_saw_hash);
	transition(STATE_DEFAULT, PP_CHAR_CLASS_eof, PP_STATE_eof);

	// whitespace stays in leading whitespace
	transition(PP_STATE_in_leading_whitespace, PP_CHAR_CLASS_whitespace, PP_STATE_in_leading_whitespace);

	// anytime we see whitespace after a newline, we go to leading whitespace
	transition(PP_STATE_saw_cr, PP_CHAR_CLASS_whitespace, PP_STATE_in_leading_whitespace);
	transition(PP_STATE_saw_lf, PP_CHAR_CLASS_whitespace, PP_STATE_in_leading_whitespace);

	// lf after cr
	transition(PP_STATE_saw_cr, PP_CHAR_CLASS_lf, PP_STATE_in_leading_whitespace);
#ifndef TREAT_LF_CR_AS_TWO_NEWLINES
	transition(PP_STATE_saw_lf, PP_CHAR_CLASS_cr, PP_STATE_in_leading_whitespace);
#endif

	// leading whitespace tracking is so we can detect # after leading whitespace
	transition(PP_STATE_in_leading_whitespace, PP_CHAR_CLASS_hash, PP_STATE_saw_leading_hash);
	transition(PP_STATE_saw_cr, PP_CHAR_CLASS_hash, PP_STATE_saw_leading_hash);
	transition(PP_STATE_saw_lf, PP_CHAR_CLASS_hash, PP_STATE_saw_leading_hash);

	// comments
	transition(PP_STATE_saw_slash, PP_CHAR_CLASS_slash, PP_STATE_comment_singleline);
	transition(PP_STATE_saw_slash, PP_CHAR_CLASS_star, PP_STATE_comment_multiline);

	// single-line parse
	transition(PP_STATE_comment_singleline, DEFAULT, PP_STATE_comment_singleline);
	transition(PP_STATE_comment_singleline, PP_CHAR_CLASS_eof, PP_STATE_eof);
	transition(PP_STATE_comment_singleline, PP_CHAR_CLASS_cr, PP_STATE_saw_cr);
	transition(PP_STATE_comment_singleline, PP_CHAR_CLASS_lf, PP_STATE_saw_lf);

	// multi-line parse
	transition(STATE_ANY_MULTILINE_COMMENT, DEFAULT, PP_STATE_comment_multiline);
	transition(STATE_ANY_MULTILINE_COMMENT, PP_CHAR_CLASS_cr, PP_STATE_comment_multiline_seen_cr);
	transition(STATE_ANY_MULTILINE_COMMENT, PP_CHAR_CLASS_lf, PP_STATE_comment_multiline_seen_lf);
	transition(STATE_ANY_MULTILINE_COMMENT, PP_CHAR_CLASS_star, PP_STATE_comment_multiline_seen_star);
	transition(STATE_ANY_MULTILINE_COMMENT, PP_CHAR_CLASS_eof, PP_STATE_error_unterminated_multiline_comment);
	transition(PP_STATE_comment_multiline_seen_star, PP_CHAR_CLASS_slash, PP_STATE_ready);
	transition(PP_STATE_comment_multiline_seen_cr, PP_CHAR_CLASS_lf, PP_STATE_comment_multiline);  // don't double-count CR-LF
#ifndef TREAT_LF_CR_AS_TWO_NEWLINES
	transition(PP_STATE_comment_multiline_seen_lf, PP_CHAR_CLASS_cr, PP_STATE_comment_multiline);  // don't double-count LF-CR
#endif

	transition(PP_STATE_saw_period, PP_CHAR_CLASS_digit, PP_STATE_in_number);
	transition(PP_STATE_saw_period, PP_CHAR_CLASS_idtype, PP_STATE_saw_identifier_start);

	// parse preprocessor number

	transition(PP_STATE_in_number, PP_CHAR_CLASS_digit, PP_STATE_in_number);
	transition(PP_STATE_in_number, PP_CHAR_CLASS_period, PP_STATE_in_number);
	transition(PP_STATE_in_number, PP_CHAR_CLASS_idtype, PP_STATE_in_number);
	transition(PP_STATE_in_number, PP_CHAR_CLASS_idtype_e_E, PP_STATE_in_number_saw_e);

	transition(PP_STATE_in_number_saw_e, PP_CHAR_CLASS_plusminus, PP_STATE_in_number);
	transition(PP_STATE_in_number_saw_e, PP_CHAR_CLASS_digit, PP_STATE_in_number);
	transition(PP_STATE_in_number_saw_e, PP_CHAR_CLASS_period, PP_STATE_in_number);
	transition(PP_STATE_in_number_saw_e, PP_CHAR_CLASS_idtype, PP_STATE_in_number);
	transition(PP_STATE_in_number_saw_e, PP_CHAR_CLASS_idtype_e_E, PP_STATE_in_number_saw_e);

	transition(PP_STATE_in_string_literal, DEFAULT, PP_STATE_in_string_literal);
	transition(PP_STATE_in_string_literal, PP_CHAR_CLASS_quotation, PP_STATE_ready);
	transition(PP_STATE_in_string_literal, PP_CHAR_CLASS_backslash, PP_STATE_in_string_literal_saw_backslash);
	transition(PP_STATE_in_string_literal_saw_backslash, DEFAULT, PP_STATE_in_string_literal);

	transition(PP_STATE_in_char_literal, DEFAULT, PP_STATE_in_char_literal);
	transition(PP_STATE_in_char_literal, PP_CHAR_CLASS_apostrophe, PP_STATE_ready);
	transition(PP_STATE_in_char_literal, PP_CHAR_CLASS_backslash, PP_STATE_in_char_literal_saw_backslash);
	transition(PP_STATE_in_char_literal_saw_backslash, DEFAULT, PP_STATE_in_char_literal);

	// init-scan state machine transitions
	// by default, all states terminate scan
	for (i = 0; i < PP_STATE_active_count; ++i)
		initscan_transition(i, DEFAULT, PP_WHITESCAN_STATE_stop_seen_onechar);

	// every character class has a default behavior:
	initscan_transition(STATE_DEFAULT, PP_CHAR_CLASS_whitespace, PP_WHITESCAN_STATE_whitespace);
	initscan_transition(STATE_DEFAULT, PP_CHAR_CLASS_cr, PP_WHITESCAN_STATE_saw_cr);
	initscan_transition(STATE_DEFAULT, PP_CHAR_CLASS_lf, PP_WHITESCAN_STATE_saw_lf);
	initscan_transition(STATE_DEFAULT, PP_CHAR_CLASS_slash, PP_WHITESCAN_STATE_saw_slash);

	// lf after cr
	initscan_transition(PP_WHITESCAN_STATE_saw_cr, PP_CHAR_CLASS_lf, PP_WHITESCAN_STATE_whitespace);
#ifndef TREAT_LF_CR_AS_TWO_NEWLINES
	initscan_transition(PP_WHITESCAN_STATE_saw_lf, PP_CHAR_CLASS_cr, PP_WHITESCAN_STATE_whitespace);
#endif

	// comments
	initscan_transition(PP_WHITESCAN_STATE_saw_slash, DEFAULT, PP_WHITESCAN_STATE_stop_seen_twochars);
	initscan_transition(PP_WHITESCAN_STATE_saw_slash, PP_CHAR_CLASS_slash, PP_WHITESCAN_STATE_comment_singleline);
	initscan_transition(PP_WHITESCAN_STATE_saw_slash, PP_CHAR_CLASS_star, PP_WHITESCAN_STATE_comment_multiline);

	// single-line parse
	initscan_transition(PP_WHITESCAN_STATE_comment_singleline, DEFAULT, PP_WHITESCAN_STATE_comment_singleline);
	initscan_transition(PP_WHITESCAN_STATE_comment_singleline, PP_CHAR_CLASS_cr, PP_WHITESCAN_STATE_saw_cr);
	initscan_transition(PP_WHITESCAN_STATE_comment_singleline, PP_CHAR_CLASS_lf, PP_WHITESCAN_STATE_saw_lf);

	// multi-line parse
	initscan_transition(STATE_ANY_MULTILINE_COMMENT, DEFAULT, PP_WHITESCAN_STATE_comment_multiline);
	initscan_transition(STATE_ANY_MULTILINE_COMMENT, PP_CHAR_CLASS_cr, PP_WHITESCAN_STATE_comment_multiline_seen_cr);
	initscan_transition(STATE_ANY_MULTILINE_COMMENT, PP_CHAR_CLASS_lf, PP_WHITESCAN_STATE_comment_multiline_seen_lf);
	initscan_transition(STATE_ANY_MULTILINE_COMMENT, PP_CHAR_CLASS_star, PP_WHITESCAN_STATE_comment_multiline_seen_star);
	initscan_transition(PP_WHITESCAN_STATE_comment_multiline_seen_star, PP_CHAR_CLASS_slash, PP_WHITESCAN_STATE_whitespace);
	initscan_transition(PP_WHITESCAN_STATE_comment_multiline_seen_cr, PP_CHAR_CLASS_lf, PP_WHITESCAN_STATE_comment_multiline);	// don't double-count CR-LF
#ifndef TREAT_LF_CR_AS_TWO_NEWLINES
	initscan_transition(PP_WHITESCAN_STATE_comment_multiline_seen_lf, PP_CHAR_CLASS_cr, PP_WHITESCAN_STATE_comment_multiline);	// don't double-count LF-CR
#endif

	initscan_transition(STATE_DEFAULT, PP_CHAR_CLASS_eof, PP_WHITESCAN_STATE_stop_eof);
	initscan_transition(STATE_ANY_MULTILINE_COMMENT, PP_CHAR_CLASS_eof, PP_WHITESCAN_STATE_error_unterminated_multiline_comment);

	// properties of transitions (stored as properties of end state)
	memset(pp_initscan_state_is_end_of_line, 0, sizeof(pp_initscan_state_is_end_of_line));
	pp_initscan_state_is_end_of_line[PP_WHITESCAN_STATE_saw_cr] = 1;
	pp_initscan_state_is_end_of_line[PP_WHITESCAN_STATE_saw_lf] = 1;
	pp_initscan_state_is_end_of_line[PP_WHITESCAN_STATE_comment_multiline_seen_cr] = 1;
	pp_initscan_state_is_end_of_line[PP_WHITESCAN_STATE_comment_multiline_seen_lf] = 1;
}


struct macro_definition* pp_define(struct stb_arena* a, const char* p)
{
	struct macro_definition* m;
	pp_context c = {0};
	parse_state ps = {0};
	c.macro_arena = *a;
	ps.filename = NULL;
	ps.src_line_number = 0;
	ps.parent = 0;
	ps.context = &c;
	ps.state_limit = PP_STATE_active_count;
	m = create_macro_definition(&ps, p);
	*a = c.macro_arena;
	return m;
}

static void define_macro(pphash* map, struct macro_definition* m)
{
	stringhash_put(map, m->symbol_name, m->symbol_name_length, m);
}

struct macro_definition* pp_define_custom_macro(struct stb_arena* a, const char* identifier, unsigned char preprocess_args_first)
{
	struct macro_definition* md = (struct macro_definition*)stb_arena_alloc(a, sizeof(*md));

	memset(md, 0, sizeof(*md));
	md->symbol_name = (char*)identifier;
	md->symbol_name_length = strlen(identifier);
	md->simple_expansion = 0;
	md->simple_expansion_length = 0;
	md->num_parameters = MACRO_NUM_PARAMETERS_custom;
	md->predefined = 1;
	md->preprocess_args_first = preprocess_args_first;

	return md;
}

char* preprocess_file(const char* filename,
	void* custom_context,
	struct macro_definition** predefined_macros,
	int num_predefined_macros,
	pp_diagnostic** pd,
	int* num_pd,
	char* output_inlinebuffer,
	size_t output_inlinebuffersize)
{
	char* output = 0;
	pp_context c = { 0 };
	int i;
	size_t length;
	const char* main_file;

	if (output_inlinebuffer && output_inlinebuffersize)
	{
		output = (char*)stbds_arrinlinef((size_t*)output_inlinebuffer, sizeof(char), output_inlinebuffersize - sizeof(stbds_array_header));
	}

	main_file = (*loadfile_callback)(filename, custom_context, &length);
	*num_pd = 0;

	if (main_file == 0)
	{
		parse_state ps = {0};
		ps.filename = "<internal>";
		ps.src_line_number = 0;
		ps.context = &c;
		do_error(&ps, "Couldn't open '%s'.", filename);
		*pd = c.diagnostics;
		*num_pd = (int)arrlen(c.diagnostics);
		return NULL;
	}

	c.custom_context = custom_context;

	// allocate the macro definition dictionary very large so (a) it's unlikely to grow,
	// and (b) so most things hit in there first slot. but going too much bigger can
	// backfire by slowing down the initialization
	stringhash_create(&c.macro_map, 4096);
	
	// enable string arena allocation for undef_map, as the map is passed stack memory identifiers that need to be duplicated
	stbds_sh_new_arena(c.undef_map);

	memset(c.macro_bloom_filter, 0, sizeof(c.macro_bloom_filter));

	for (i = 0; i < arrlen(predefined_macros); ++i)
		define_macro(&c.macro_map, predefined_macros[i]);

	define_macro(&c.macro_map, &predefined_defined);
	define_macro(&c.macro_map, &predefined_FILE);
	define_macro(&c.macro_map, &predefined_LINE);
	define_macro(&c.macro_map, &predefined_COUNTER);

	for (i = 0; i < arrlen(c.macro_map.pair); ++i)
		update_macro_filter(&c, c.macro_map.pair[i].key, c.macro_map.pair[i].key_length);

	{
		// initial parse_state for preprocess_string_from_file to initialize from
		parse_state ps = {0};
		ps.context = &c;
		ps.dest = output;
		ps.state_limit = PP_STATE_active_count;

		output = preprocess_string_from_file(&ps, filename, main_file, length, 0, 0);
		arrpush(output, 0);
	}

	(*freefile_callback)(filename, main_file, custom_context);

	for (i = 0; i < arrlen(c.macro_map.pair); ++i)
	{
		if (c.macro_map.pair[i].key)
		{
			struct macro_definition* md = (struct macro_definition*)c.macro_map.pair[i].value;
			if (!md->predefined)
			{
				macro_free(md);
			}
		}
	}

	shfree(c.once_map);
	shfree(c.undef_map);
	arrfree(c.ifdef_stack);
	stringhash_destroy(&c.macro_map);
	stb_arena_free(&c.macro_arena);

	if (c.diagnostics)
	{
		*pd = c.diagnostics;
		*num_pd = (int)arrlen(*pd);
		if (c.stop)
		{
			arrfree(output);
			output = NULL;
		}
	}
	else
		*pd = 0;

	// printf("%d lines (%d of them disabled by #if), not counting %d files skipped by #pragma once or include guards, %d includes\n", c.num_lines,
	// c.num_disabled_lines, c.num_onced_files, c.num_includes);

	return output;
}

int preprocessor_file_size(char* text)
{
	return text ? arrlen(text) : 0;
}
int preprocessor_file_capacity(char* text)
{
	return text ? arrcap(text) : 0;
}

char* preprocessor_file_append(char* text, const char* appended_text, int appended_text_len)
{
	if (text)
	{
		int text_len = arrlen(text);

		// The preprocessor text array length includes the null terminator, so we don't need to add one here
		arrsetlen(text, text_len + appended_text_len);

		// Subtract one, so we start writing at original null terminator
		memcpy(text + text_len - 1, appended_text, appended_text_len);

		// And add a new null terminator
		text[text_len + appended_text_len - 1] = 0;
	}

	return text;
}

void preprocessor_file_free(char* text, pp_diagnostic* pd)
{
	int i, j;
	arrfree(text);
	for (i = 0; i < arrlen(pd); ++i)
	{
#pragma warning(push)
// MSVC analysis incorrectly reports that where.filename could be uninitialized, but there's no path
// in which an uninitialized pp_where can be added to the array, so this is bogus
#pragma warning(disable:6001)
		for (j = 0; j < arrlen(pd[i].where); ++j)
			STB_COMMON_FREE(pd[i].where[j].filename);
#pragma warning(pop)
		arrfree(pd[i].where);
		STB_COMMON_FREE(pd[i].message);
	}
	arrfree(pd);
}

static void init_directive(char* s, int hash)
{
	int sum = 0;
	int i;
	for (i = 0; s[i] != 0; ++i)
		sum += (uint8)s[i];
	sum &= (DIRECTIVE_HASH_SIZE - 1);
	if (sum != hash)
	{
		fprintf(stderr, "Preprocessor fatal internal error: hash initialization mismatch for '%s' (%s %d).\n", s, __FILE__, __LINE__);
		exit(1);
	}
	if (directive_hash[hash].name[0] != 0)
	{
		fprintf(stderr, "Preprocessor fatal internal error: hash collision between '%s' and '%s' (%s %d).\n", s, directive_hash[hash].name, __FILE__, __LINE__);
		exit(1);
	}

	// name member is sized to fit the largest directive (7 characters plus null) -- use strncpy to avoid warnings
	strncpy(directive_hash[hash].name, s, sizeof(directive_hash[hash].name));
	directive_hash[hash].name_len = strlen(s);
}

static void init_char_type(void)
{
	int i;
	memset(pp_char_flags1, 0, 256);
	memset(pp_char_flags2, 0, 256);

	for (i = 1; i < 127; ++i)
	{
		if (isalpha(i) || i == '_' || i == '$')
			pp_char_flags1[i] |= CHAR1_IS_PP_IDENTIFIER_FIRST;
		if (isdigit(i))
			pp_char_flags1[i] |= CHAR1_IS_DIGIT;
	}

	pp_char_flags1[0] |= CHAR1_IS_NUL;
	pp_char_flags1['\n'] |= CHAR1_IS_NEWLINE;
	pp_char_flags1['\r'] |= CHAR1_IS_NEWLINE;
	pp_char_flags1[' '] |= CHAR1_IS_NONNEWLINE_WHITESPACE;
	pp_char_flags1['\t'] |= CHAR1_IS_NONNEWLINE_WHITESPACE;

	pp_char_flags1['('] |= CHAR1_IS_BALANCE_SCAN_ACTION;
	pp_char_flags1[')'] |= CHAR1_IS_BALANCE_SCAN_ACTION;
	pp_char_flags1[','] |= CHAR1_IS_BALANCE_SCAN_ACTION;
	pp_char_flags1['/'] |= CHAR1_IS_BALANCE_SCAN_ACTION;
	pp_char_flags1['\\'] |= CHAR1_IS_BALANCE_SCAN_ACTION;

	pp_char_flags1['\''] |= CHAR1_IS_QUOTE;
	pp_char_flags1['\"'] |= CHAR1_IS_QUOTE;
	pp_char_flags1['\\'] |= CHAR1_IS_BACKSLASH;

	pp_char_flags2[0] |= CHAR2_IS_NUL;
	pp_char_flags2['\n'] |= CHAR2_IS_NEWLINE;
	pp_char_flags2['\r'] |= CHAR2_IS_NEWLINE;
	pp_char_flags2['/'] |= CHAR2_IS_SLASH;
}



void init_preprocessor(
	loadfile_callback_func load_callback, 
	freefile_callback_func free_callback,
	resolveinclude_callback_func resolve_callback,
	custommacro_begin_callback_func custommacro_begin_callback,
	custommacro_end_callback_func custommacro_end_callback)
{
#ifdef HASHTEST
	test_stringhash();
#endif

	init_preprocessor_scanner();
	init_char_type();

	init_directive("define", HASH_define);
	init_directive("else", HASH_else);
	init_directive("elif", HASH_elif);
	init_directive("endif", HASH_endif);
	init_directive("error", HASH_error);
	init_directive("if", HASH_if);
	init_directive("ifdef", HASH_ifdef);
	init_directive("ifndef", HASH_ifndef);
	init_directive("include", HASH_include);
	init_directive("line", HASH_line);
	init_directive("pragma", HASH_pragma);
	init_directive("undef", HASH_undef);
	init_directive("warning", HASH_warning);

	loadfile_callback = load_callback;
	freefile_callback = free_callback;
	resolveinclude_callback = resolve_callback;
	custommacro_begin = custommacro_begin_callback;
	custommacro_end = custommacro_end_callback;
}

#pragma warning(pop)
