// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include <stdlib.h>

enum
{
	PP_RESULT_ok,
	PP_RESULT_supplementary,			 // extra information associated with previous warning/error
	PP_RESULT_undef_of_undefined_macro,	 // by default this is not an error
	PP_RESULT_explicit_warning,
	
	PP_RESULT_internal_error_out_of_memory,

	PP_RESULT_ERROR = 5,

	PP_RESULT_counter_overflowed,
	PP_RESULT_too_many_arguments_to_macro,
	PP_RESULT_invalid_char,
	PP_RESULT_unrecognized_directive,
	PP_RESULT_directive_not_at_start_of_line,
	PP_RESULT_unexpected_character_in_directive,
	PP_RESULT_unexpected_character_after_end_of_directive,
	PP_RESULT_undef_of_predefined_macro,
	PP_RESULT_identifier_too_long,
	PP_RESULT_malformed_pragma_message,
	PP_RESULT_malformed_line_filename,
	PP_RESULT_invalid_line_linenumber,
	PP_RESULT_syntax_error_conditional_expression,
	PP_RESULT_conditional_expression_overflow,
	PP_RESULT_conditional_expression_division_by_zero,

	PP_RESULT_count
};

enum
{
	PP_RESULT_MODE_error,  // default error type is 0 to simplify initialization
	PP_RESULT_MODE_warning,
	PP_RESULT_MODE_supplementary,

	PP_RESULT_MODE_warning_fast,
	PP_RESULT_MODE_no_warning
};


struct macro_definition;
struct stb_arena;

typedef struct
{
	char* filename;
	int line_number;
	int column;
} pp_where;

typedef struct
{
	char* message;
	int diagnostic_code;
	int error_level;
	pp_where* where;
} pp_diagnostic;


#ifdef __cplusplus
#define STB_PP_DEF extern "C"
#else
#define STB_PP_DEF extern
#endif

// callback function required to be passed to init_preprocessor for implementation of file loading functionality
typedef const char* (*loadfile_callback_func)(const char* filename, void* custom_context, size_t* out_length);

// callback function required to be passed to init_preprocessor for implementation of freeing loaded files
typedef void (*freefile_callback_func)(const char* filename, const char* loaded_file, void* custom_context);

// callback function required to be passed to init_preprocessor for implementation of include resolution
// the callback is expected to handle both absolute and relative paths (the latter relative to the given
// "parent" file path)
// note: returned resolved paths must remain valid for the lifetime of a single preprocessor execution
typedef const char* (*resolveinclude_callback_func)(const char* path, unsigned int path_len, const char* parent, void* custom_context);

// Callback functions for custom macros.  Custom macros return their own substitution text, which is then further preprocessed.
// The substitution text must include 15 bytes of padding past the null terminator, as the preprocessor uses SSE reads which may
// read past the null terminator.  The text passed to the callback includes the macro identifier and its arguments, with backslashes
// removed.  A function is called when entering the macro, and again when complete, allowing stateful logic based on whether
// parsing is inside a certain custom macro.  Any custom macro will automatically be disabled inside itself, with the idea that
// this allows a singleton buffer for the substitution text, as there's no concern about nested calls.  If text is dynamically
// allocated, the caller is responsible for freeing it.  It's also valid to return the original text passed in as the substitution
// text, if you just want to set some state inside the context, and not actually change the macro text -- since the macro is
// disabled, it will then be echoed as non-macro text.
typedef const char* (*custommacro_begin_callback_func)(const char* original_text, void* custom_context);
typedef void (*custommacro_end_callback_func)(const char* original_text, void* custom_context, const char* substitution_text);

// init function must be called once before any invocations of preprocess_file
STB_PP_DEF void init_preprocessor(
	loadfile_callback_func load_callback,
	freefile_callback_func free_callback,
	resolveinclude_callback_func resolve_callback,
	custommacro_begin_callback_func custommacro_begin_callback,
	custommacro_end_callback_func custommacro_end_callback);

// main preprocessing execution function; returns the preprocessed string and sets any diagnostic messages
// in the pd array (setting num_pd to the diagnostic count). preprocessor_file_free should be called for 
// each invocation to free any allocated memory.
STB_PP_DEF char* preprocess_file(
	const char* filename,
	void* custom_context,
	struct macro_definition** predefined_macros,
	int num_predefined_macros,
	pp_diagnostic** pd,
	int* num_pd,
	char* output_inlinebuffer,
	size_t output_inlinebuffersize);

STB_PP_DEF int preprocessor_file_size(char* text);
STB_PP_DEF int preprocessor_file_capacity(char* text);

// Append text to the end of a file generated with preprocess_file.  Can save reallocation overhead relative
// to appending the text later, as there will usually be slack space available.
STB_PP_DEF char* preprocessor_file_append(char* text, const char* appended_text, int appended_text_len);

// frees memory allocated by preprocess_file (preprocessed results and diagnostic messages)
STB_PP_DEF void preprocessor_file_free(char* text, pp_diagnostic* pd);

// constructs a preprocessor definition, allocating on the given arena. "def" should be in the format:
// DEFINE_NAME DEFINE_VALUE
STB_PP_DEF struct macro_definition* pp_define(struct stb_arena* a, const char* def);

// specifies a custom macro name, with substitution to be handled by the custom macro callback functions.  If "preprocess_args_first" is set,
// args of macro are preprocessed before begin callback (and again afterwards).
STB_PP_DEF struct macro_definition* pp_define_custom_macro(struct stb_arena* a, const char* identifier, unsigned char preprocess_args_first);

// override behaviour of a particular error result (can be used to, for instance, downgrade
// an error to a warning or a "no warning" i.e. valid case).
STB_PP_DEF void pp_set_warning_mode(int result_code, int result_mode);

// hash function for string of length 'len', roughly xxhash32
STB_PP_DEF unsigned int preprocessor_hash(const char* data, size_t len);
