// Copyright Epic Games, Inc. All Rights Reserved.

#include "stb_alloc.h"
#include <string.h>

void* stb_arena_alloc_aligned(struct stb_arena* a, size_t size, size_t align)
{
	void* p;
	a->last_block_alloc_offset = (a->last_block_alloc_offset + align - 1) & ~(align - 1);
	if (a->last_block_alloc_offset + size > a->last_block_size)
	{
		// initialize the block size if it's uninitialized
		if (a->default_block_size == 0)
			a->default_block_size = 8192;
		if (a->block_size == 0)
			a->block_size = a->default_block_size;

		// increase the block size every 4 allocations
		if ((a->num_blocks & 3) == 3)
			a->block_size *= 2;

		// the most recent block may be larger if the allocation itself is too large
		a->last_block_size = a->block_size;
		if (a->last_block_size < size)
			a->last_block_size = size;

		// grow the list of blocks if necessary
		if (a->num_blocks == 0)
		{
			a->blocks = (void**)STB_COMMON_MALLOC(sizeof(a->blocks[0]) * 32);
		}
		else if (a->num_blocks >= 32)
		{
			// if number of blocks is a power of two
			if ((a->num_blocks & (a->num_blocks - 1)) == 0)
			{
				// grow table
				void* ptr = STB_COMMON_REALLOC(a->blocks, sizeof(a->blocks[0]) * a->num_blocks * 2);
				if (ptr)
					a->blocks = (void**)ptr;
			}
		}

		// allocate and record allocation.  31 bytes for alignment, 16 bytes for SSE padding at end of a block
		a->last_block = (unsigned char*)STB_COMMON_MALLOC(a->last_block_size + 31 + 16);
		STB_ASSUME(a->blocks != NULL);
		a->blocks[a->num_blocks++] = a->last_block;
		a->last_block_alloc_offset = 0;

		{
			// align allocation to multiple of 32 for future-proofing
			union
			{
				void* p;
				size_t val;
			} convert;
			convert.p = a->last_block;
			if (convert.val & 31)
			{
				a->last_block += 32 - (convert.val & 31);
			}
		}
	}

	p = a->last_block + a->last_block_alloc_offset;
	a->last_block_alloc_offset += size;
	return p;
}

static size_t stb_arena_align_size(size_t size)
{
	if ((size & 7) != 0)
		return 4;
	if (size >= 16)
	{
		if (size >= 32 && (size & 31) == 0)
			return 32;
		if ((size & 15) == 0)
			return 16;
	}
	return 8;
}

void* stb_arena_alloc(struct stb_arena* a, size_t size)
{
	return stb_arena_alloc_aligned(a, size, stb_arena_align_size(size));
}

char* stb_arena_alloc_string_length(struct stb_arena* a, const char* str, size_t length)
{
	char* p = (char*)stb_arena_alloc_aligned(a, length + 1, 4);
	memcpy(p, str, length);
	p[length] = 0;
	return p;
}

char* stb_arena_alloc_string(struct stb_arena* a, const char* str)
{
	return stb_arena_alloc_string_length(a, str, strlen(str));
}

void stb_arena_free(struct stb_arena* a)
{
	int i;
	for (i = 0; i < a->num_blocks; ++i)
		STB_COMMON_FREE(a->blocks[i]);
	STB_COMMON_FREE(a->blocks);
	a->blocks = 0;
	a->num_blocks = 0;
	a->last_block_size = 0;
	a->last_block = 0;
	a->block_size = 0;
}