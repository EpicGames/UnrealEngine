// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// @TODO rewrite to use a linked list of blocks instead of an array

#include <stdlib.h>

#include "stb_common.h"

#ifdef __cplusplus
#define STB_ALLOC_DEF extern "C"
#else
#define STB_ALLOC_DEF extern
#endif

struct stb_arena;

// allocate from an arena
STB_ALLOC_DEF void* stb_arena_alloc(struct stb_arena* a, size_t size);
STB_ALLOC_DEF void* stb_arena_alloc_aligned(struct stb_arena* a, size_t size, size_t align);

// allocate a string from an arena
STB_ALLOC_DEF char* stb_arena_alloc_string(struct stb_arena* a, const char* str);
STB_ALLOC_DEF char* stb_arena_alloc_string_length(struct stb_arena* a, const char* str, size_t length);

// free the entire arena, leaves arena reset so it can be used again
STB_ALLOC_DEF void stb_arena_free(struct stb_arena* a);

// You allocate this structure yourself; initialize to all 0s and
// don't mess with the innards. Optionally, you can initialize the first
// field to the default block size you want, e.g.
//     stb_arena arena = { 65536 };
struct stb_arena
{
	size_t default_block_size;
	size_t block_size;
	void** blocks;
	unsigned char* last_block;
	int num_blocks;
	size_t last_block_alloc_offset;
	size_t last_block_size;
};

