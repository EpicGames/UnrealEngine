// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define STB_COMMON_XSTR(s) STB_COMMON_STR(s)
#define STB_COMMON_STR(s) #s
#ifdef STB_CONFIG
#include STB_COMMON_XSTR(STB_CONFIG)
#endif

// The following macros can be defined to override memory allocation for preprocessor/stbds operation:
// STB_COMMON_MALLOC
// STB_COMMON_REALLOC
// STB_COMMON_FREE
// STB_COMMON_STRDUP
// Note that overriding these is all-or-nothing and so will error if only some are defined.
// By default (if not specified) malloc/realloc/free/strdup will be used.
#if !defined(STB_COMMON_MALLOC) && !defined(STB_COMMON_REALLOC) && !defined(STB_COMMON_FREE) && !defined(STB_COMMON_STRDUP)
#include <stdlib.h>
#define STB_COMMON_MALLOC(s) malloc(s)
#define STB_COMMON_REALLOC(p, s) realloc(p, s)
#define STB_COMMON_FREE(p) free(p)
#define STB_COMMON_STRDUP(s) strdup(s)
#elif defined(STB_COMMON_MALLOC) && defined(STB_COMMON_REALLOC) && defined(STB_COMMON_FREE) && defined(STB_COMMON_STRDUP)
// all are defined
#else
#error "You must define all or none of STB_COMMON_MALLOC/STB_COMMON_REALLOC/STB_COMMON_FREE/STB_COMMON_STRDUP"
#endif

#if defined(_Analysis_assume_)
#define STB_ASSUME(expr) _Analysis_assume_(expr)
#else
#define STB_ASSUME(expr)
#endif

// The following macros can be defined to override assertion handling:
// STB_COMMON_ASSERT(expr)
// STB_COMMON_ASSERTF(expr, fmt, ...)     -- printf-style message describing the failure
// Both forms must be defined together if overriding; the override receives a printf-style format string and varargs
// suitable for routing to a host application's diagnostic system.
// By default (if not specified) <assert.h> is used and the format string is discarded.
#if !defined(STB_COMMON_ASSERT) && !defined(STB_COMMON_ASSERTF)
#include <assert.h>
#define STB_COMMON_ASSERT(expr) assert(expr)
#define STB_COMMON_ASSERTF(expr, fmt, ...) assert(expr)
#elif defined(STB_COMMON_ASSERT) && defined(STB_COMMON_ASSERTF)
// both are defined
#else
#error "You must define both or neither of STB_COMMON_ASSERT/STB_COMMON_ASSERTF"
#endif