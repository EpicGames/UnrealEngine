// Copyright Epic Games, Inc. All Rights Reserved.

#include "stb_ds.h"
#include <assert.h>
#include <string.h>

#ifndef STBDS_ASSERT
#define STBDS_ASSERT_WAS_UNDEFINED
#define STBDS_ASSERT(x) ((void)0)
#endif

#ifdef STBDS_STATISTICS
#define STBDS_STATS(x) x
size_t stbds_array_grow;
size_t stbds_hash_grow;
size_t stbds_hash_shrink;
size_t stbds_hash_rebuild;
size_t stbds_hash_probes;
size_t stbds_hash_alloc;
size_t stbds_rehash_probes;
size_t stbds_rehash_items;
#else
#define STBDS_STATS(x)
#endif

//
// stbds_arr implementation
//

// int *prev_allocs[65536];
// int num_prev;

void* stbds_arrinlinef(size_t* buf, size_t elemsize, size_t elemcount)
{
	stbds_array_header* h = (stbds_array_header*)buf;

	h->length = 0;
	h->capacity = elemcount;
	h->hash_table = 0;
	h->temp = 0;
	h->elemsize = elemsize;
	h->inlinealloc = 1;

	return h + 1;
}

void* stbds_arrinline_suballocf(void* a, size_t min_capacity)
{
	if (!a || !stbds_header(a)->inlinealloc)
	{
		return 0;
	}

	stbds_array_header* h = stbds_header(a);

	// Get the end of the existing array and buffer from the capacity -- used_end is rounded upward to word size
	char* used_end = (char*)a + ((h->elemsize * h->length + sizeof(size_t) - 1) & ~(sizeof(size_t) - 1));
	char* capacity_end = (char*)a + h->elemsize * h->capacity;

	// Set existing allocation capacity equal to length, so no more can be allocated from the inline buffer.
	// Growing will still work, it will just convert the inline allocation to a dynamic allocation.
	h->capacity = h->length;

	// Determine whether we have the minimum required capacity for the suballocation, if not return null.
	if (capacity_end - used_end < sizeof(stbds_array_header) + min_capacity * h->elemsize)
	{
		return 0;
	}

	// Set up a new buffer
	stbds_array_header* h_suballoc = (stbds_array_header*)used_end;

	h_suballoc->length = 0;
	h_suballoc->capacity = ((capacity_end - used_end) - sizeof(stbds_array_header)) / h->elemsize;
	h_suballoc->hash_table = 0;
	h_suballoc->temp = 0;
	h_suballoc->elemsize = h->elemsize;
	h_suballoc->inlinealloc = 1;

	return h_suballoc + 1;
}

void* stbds_arrgrowf(void* a, size_t elemsize, size_t addlen, size_t min_cap)
{
	stbds_array_header temp = {0};	// force debugging
	void* b;
	size_t min_len = stbds_arrlen(a) + addlen;
	(void)sizeof(temp);

	// compute the minimum capacity needed
	if (min_len > min_cap)
		min_cap = min_len;

	if (min_cap <= stbds_arrcap(a))
		return a;

	// increase needed capacity to guarantee O(1) amortized
	if (min_cap < 2 * stbds_arrcap(a))
		min_cap = 2 * stbds_arrcap(a);
	else if (min_cap < 4)
		min_cap = 4;

	// if (num_prev < 65536) if (a) prev_allocs[num_prev++] = (int *) ((char *) a+1);
	// if (num_prev == 2201)
	//   num_prev = num_prev;
	if (a && stbds_header(a)->inlinealloc)
	{
		b = STB_COMMON_MALLOC(elemsize * min_cap + sizeof(stbds_array_header));
		STB_ASSUME(b != NULL);
		memcpy(b, stbds_header(a), elemsize * stbds_header(a)->length + sizeof(stbds_array_header));
		((stbds_array_header*)b)->inlinealloc = 0;
	}
	else
	{
		b = STB_COMMON_REALLOC((a) ? stbds_header(a) : 0, elemsize * min_cap + sizeof(stbds_array_header));
	}
	STB_ASSUME(b != NULL);

	// if (num_prev < 65536) prev_allocs[num_prev++] = (int *) (char *) b;
	b = (char*)b + sizeof(stbds_array_header);
	if (a == NULL)
	{
		stbds_header(b)->length = 0;
		stbds_header(b)->hash_table = 0;
		stbds_header(b)->temp = 0;
		stbds_header(b)->elemsize = elemsize;
		stbds_header(b)->inlinealloc = 0;
	}
	else
	{
		STBDS_STATS(++stbds_array_grow);
	}
	stbds_header(b)->capacity = min_cap;

	return b;
}

//
// stbds_hm hash table implementation
//

#ifdef STBDS_INTERNAL_SMALL_BUCKET
#define STBDS_BUCKET_LENGTH 4
#else
#define STBDS_BUCKET_LENGTH 8
#endif

#define STBDS_BUCKET_SHIFT (STBDS_BUCKET_LENGTH == 8 ? 3 : 2)
#define STBDS_BUCKET_MASK (STBDS_BUCKET_LENGTH - 1)
#define STBDS_CACHE_LINE_SIZE 64

#define STBDS_ALIGN_FWD(n, a) (((n) + (a)-1) & ~((a)-1))

typedef struct
{
	size_t hash[STBDS_BUCKET_LENGTH];
	ptrdiff_t index[STBDS_BUCKET_LENGTH];
} stbds_hash_bucket;  // in 32-bit, this is one 64-byte cache line; in 64-bit, each array is one 64-byte cache line

typedef struct
{
	char* temp_key;	 // this MUST be the first field of the hash table
	size_t slot_count;
	size_t used_count;
	size_t used_count_threshold;
	size_t used_count_shrink_threshold;
	size_t tombstone_count;
	size_t tombstone_count_threshold;
	size_t seed;
	size_t slot_count_log2;
	stbds_string_arena string;
	stbds_hash_bucket* storage;	 // not a separate allocation, just 64-byte aligned storage after this struct
} stbds_hash_index;

#define STBDS_INDEX_EMPTY -1
#define STBDS_INDEX_DELETED -2
#define STBDS_INDEX_IN_USE(x) ((x) >= 0)

#define STBDS_HASH_EMPTY 0
#define STBDS_HASH_DELETED 1

#define stbds_load_32_or_64(var, temp, v32, v64_hi, v64_lo)                                          \
	temp = v64_lo ^ v32, temp <<= 16, temp <<= 16, temp >>= 16, temp >>= 16, /* discard if 32-bit */ \
		var = v64_hi, var <<= 16, var <<= 16,								 /* discard if 32-bit */ \
		var ^= temp ^ v32

#define STBDS_SIZE_T_BITS ((sizeof(size_t)) * 8)

static size_t stbds_probe_position(size_t hash, size_t slot_count, size_t slot_log2)
{
	size_t pos;
	STBDS_NOTUSED(slot_log2);
	pos = hash & (slot_count - 1);
#ifdef STBDS_INTERNAL_BUCKET_START
	pos &= ~STBDS_BUCKET_MASK;
#endif
	return pos;
}

static size_t stbds_log2(size_t slot_count)
{
	size_t n = 0;
	while (slot_count > 1)
	{
		slot_count >>= 1;
		++n;
	}
	return n;
}

static stbds_hash_index* stbds_make_hash_index(size_t slot_count, stbds_hash_index* ot)
{
	stbds_hash_index* t;
	t = (stbds_hash_index*)STB_COMMON_REALLOC(0,
		(slot_count >> STBDS_BUCKET_SHIFT) * sizeof(stbds_hash_bucket) + sizeof(stbds_hash_index) + STBDS_CACHE_LINE_SIZE - 1);
	STB_ASSUME(t != NULL);

	t->storage = (stbds_hash_bucket*)STBDS_ALIGN_FWD((size_t)(t + 1), STBDS_CACHE_LINE_SIZE);
	t->slot_count = slot_count;
	t->slot_count_log2 = stbds_log2(slot_count);
	t->tombstone_count = 0;
	t->used_count = 0;

#if 0	 // A1
  t->used_count_threshold        = slot_count*12/16; // if 12/16th of table is occupied, grow
  t->tombstone_count_threshold   = slot_count* 2/16; // if tombstones are 2/16th of table, rebuild
  t->used_count_shrink_threshold = slot_count* 4/16; // if table is only 4/16th full, shrink
#elif 1	 // A2
	// t->used_count_threshold        = slot_count*12/16; // if 12/16th of table is occupied, grow
	// t->tombstone_count_threshold   = slot_count* 3/16; // if tombstones are 3/16th of table, rebuild
	// t->used_count_shrink_threshold = slot_count* 4/16; // if table is only 4/16th full, shrink

	// compute without overflowing
	t->used_count_threshold = slot_count - (slot_count >> 2);
	t->tombstone_count_threshold = (slot_count >> 3) + (slot_count >> 4);
	t->used_count_shrink_threshold = slot_count >> 2;

#elif 0	 // B1
	t->used_count_threshold = slot_count * 13 / 16;		   // if 13/16th of table is occupied, grow
	t->tombstone_count_threshold = slot_count * 2 / 16;	   // if tombstones are 2/16th of table, rebuild
	t->used_count_shrink_threshold = slot_count * 5 / 16;  // if table is only 5/16th full, shrink
#else	 // C1
	t->used_count_threshold = slot_count * 14 / 16;		   // if 14/16th of table is occupied, grow
	t->tombstone_count_threshold = slot_count * 2 / 16;	   // if tombstones are 2/16th of table, rebuild
	t->used_count_shrink_threshold = slot_count * 6 / 16;  // if table is only 6/16th full, shrink
#endif
	// Following statistics were measured on a Core i7-6700 @ 4.00Ghz, compiled with clang 7.0.1 -O2
	// Note that the larger tables have high variance as they were run fewer times
	//     A1            A2          B1           C1
	//    0.10ms :     0.10ms :     0.10ms :     0.11ms :      2,000 inserts creating 2K table
	//    0.96ms :     0.95ms :     0.97ms :     1.04ms :     20,000 inserts creating 20K table
	//   14.48ms :    14.46ms :    10.63ms :    11.00ms :    200,000 inserts creating 200K table
	//  195.74ms :   196.35ms :   203.69ms :   214.92ms :  2,000,000 inserts creating 2M table
	// 2193.88ms :  2209.22ms :  2285.54ms :  2437.17ms : 20,000,000 inserts creating 20M table
	//   65.27ms :    53.77ms :    65.33ms :    65.47ms : 500,000 inserts & deletes in 2K table
	//   72.78ms :    62.45ms :    71.95ms :    72.85ms : 500,000 inserts & deletes in 20K table
	//   89.47ms :    77.72ms :    96.49ms :    96.75ms : 500,000 inserts & deletes in 200K table
	//   97.58ms :    98.14ms :    97.18ms :    97.53ms : 500,000 inserts & deletes in 2M table
	//  118.61ms :   119.62ms :   120.16ms :   118.86ms : 500,000 inserts & deletes in 20M table
	//  192.11ms :   194.39ms :   196.38ms :   195.73ms : 500,000 inserts & deletes in 200M table

	if (slot_count <= STBDS_BUCKET_LENGTH)
		t->used_count_shrink_threshold = 0;
	// to avoid infinite loop, we need to guarantee that at least one slot is empty and will terminate probes
	STBDS_ASSERT(t->used_count_threshold + t->tombstone_count_threshold < t->slot_count);
	STBDS_STATS(++stbds_hash_alloc);
	if (ot)
	{
		t->string = ot->string;
		// reuse old seed so we can reuse old hashes so below "copy out old data" doesn't do any hashing
		t->seed = ot->seed;
	}
	else
	{
		size_t a, b, temp;
		memset(&t->string, 0, sizeof(t->string));
		t->seed = STB_LCG_NEXT();
	}

	{
		size_t i, j;
		for (i = 0; i < slot_count >> STBDS_BUCKET_SHIFT; ++i)
		{
			stbds_hash_bucket* b = &t->storage[i];
			for (j = 0; j < STBDS_BUCKET_LENGTH; ++j)
				b->hash[j] = STBDS_HASH_EMPTY;
			for (j = 0; j < STBDS_BUCKET_LENGTH; ++j)
				b->index[j] = STBDS_INDEX_EMPTY;
		}
	}

	// copy out the old data, if any
	if (ot)
	{
		size_t i, j;
		t->used_count = ot->used_count;
		for (i = 0; i < ot->slot_count >> STBDS_BUCKET_SHIFT; ++i)
		{
			stbds_hash_bucket* ob = &ot->storage[i];
			for (j = 0; j < STBDS_BUCKET_LENGTH; ++j)
			{
				if (STBDS_INDEX_IN_USE(ob->index[j]))
				{
					size_t hash = ob->hash[j];
					size_t pos = stbds_probe_position(hash, t->slot_count, t->slot_count_log2);
					size_t step = STBDS_BUCKET_LENGTH;
					STBDS_STATS(++stbds_rehash_items);
					for (;;)
					{
						size_t limit, z;
						stbds_hash_bucket* bucket;
						bucket = &t->storage[pos >> STBDS_BUCKET_SHIFT];
						STBDS_STATS(++stbds_rehash_probes);

						for (z = pos & STBDS_BUCKET_MASK; z < STBDS_BUCKET_LENGTH; ++z)
						{
							if (bucket->hash[z] == 0)
							{
								bucket->hash[z] = hash;
								bucket->index[z] = ob->index[j];
								goto done;
							}
						}

						limit = pos & STBDS_BUCKET_MASK;
						for (z = 0; z < limit; ++z)
						{
							if (bucket->hash[z] == 0)
							{
								bucket->hash[z] = hash;
								bucket->index[z] = ob->index[j];
								goto done;
							}
						}

						pos += step;  // quadratic probing
						step += STBDS_BUCKET_LENGTH;
						pos &= (t->slot_count - 1);
					}
				}
			done:;
			}
		}
	}

	return t;
}

#define STBDS_ROTATE_LEFT(val, n) (((val) << (n)) | ((val) >> (STBDS_SIZE_T_BITS - (n))))
#define STBDS_ROTATE_RIGHT(val, n) (((val) >> (n)) | ((val) << (STBDS_SIZE_T_BITS - (n))))

size_t stbds_hash_string(char* str, size_t seed)
{
	size_t hash = seed;
	while (*str)
		hash = STBDS_ROTATE_LEFT(hash, 9) + (unsigned char)*str++;

	// Thomas Wang 64-to-32 bit mix function, hopefully also works in 32 bits
	hash ^= seed;
	hash = (~hash) + (hash << 18);
	hash ^= hash ^ STBDS_ROTATE_RIGHT(hash, 31);
	hash = hash * 21;
	hash ^= hash ^ STBDS_ROTATE_RIGHT(hash, 11);
	hash += (hash << 6);
	hash ^= STBDS_ROTATE_RIGHT(hash, 22);
	return hash + seed;
}

#ifdef STBDS_SIPHASH_2_4
#define STBDS_SIPHASH_C_ROUNDS 2
#define STBDS_SIPHASH_D_ROUNDS 4
typedef int STBDS_SIPHASH_2_4_can_only_be_used_in_64_bit_builds[sizeof(size_t) == 8 ? 1 : -1];
#endif

#ifndef STBDS_SIPHASH_C_ROUNDS
#define STBDS_SIPHASH_C_ROUNDS 1
#endif
#ifndef STBDS_SIPHASH_D_ROUNDS
#define STBDS_SIPHASH_D_ROUNDS 1
#endif

#ifdef _MSC_VER
#pragma warning(push)
#pragma warning(disable : 4127)	 // conditional expression is constant, for do..while(0) and sizeof()==
#endif

static size_t stbds_siphash_bytes(void* p, size_t len, size_t seed)
{
	unsigned char* d = (unsigned char*)p;
	size_t i, j;
	size_t v0, v1, v2, v3, data;

	// hash that works on 32- or 64-bit registers without knowing which we have
	// (computes different results on 32-bit and 64-bit platform)
	// derived from siphash, but on 32-bit platforms very different as it uses 4 32-bit state not 4 64-bit
	v0 = ((((size_t)0x736f6d65 << 16) << 16) + 0x70736575) ^ seed;
	v1 = ((((size_t)0x646f7261 << 16) << 16) + 0x6e646f6d) ^ ~seed;
	v2 = ((((size_t)0x6c796765 << 16) << 16) + 0x6e657261) ^ seed;
	v3 = ((((size_t)0x74656462 << 16) << 16) + 0x79746573) ^ ~seed;

#ifdef STBDS_TEST_SIPHASH_2_4
	// hardcoded with key material in the siphash test vectors
	v0 ^= 0x0706050403020100ull ^ seed;
	v1 ^= 0x0f0e0d0c0b0a0908ull ^ ~seed;
	v2 ^= 0x0706050403020100ull ^ seed;
	v3 ^= 0x0f0e0d0c0b0a0908ull ^ ~seed;
#endif

#define STBDS_SIPROUND()                                   \
	do                                                     \
	{                                                      \
		v0 += v1;                                          \
		v1 = STBDS_ROTATE_LEFT(v1, 13);                    \
		v1 ^= v0;                                          \
		v0 = STBDS_ROTATE_LEFT(v0, STBDS_SIZE_T_BITS / 2); \
		v2 += v3;                                          \
		v3 = STBDS_ROTATE_LEFT(v3, 16);                    \
		v3 ^= v2;                                          \
		v2 += v1;                                          \
		v1 = STBDS_ROTATE_LEFT(v1, 17);                    \
		v1 ^= v2;                                          \
		v2 = STBDS_ROTATE_LEFT(v2, STBDS_SIZE_T_BITS / 2); \
		v0 += v3;                                          \
		v3 = STBDS_ROTATE_LEFT(v3, 21);                    \
		v3 ^= v0;                                          \
	} while (0)

	for (i = 0; i + sizeof(size_t) <= len; i += sizeof(size_t), d += sizeof(size_t))
	{
		data = d[0] | (d[1] << 8) | (d[2] << 16) | (d[3] << 24);
		data |= (size_t)(d[4] | (d[5] << 8) | (d[6] << 16) | (d[7] << 24)) << 16 << 16;	 // discarded if size_t == 4

		v3 ^= data;
		for (j = 0; j < STBDS_SIPHASH_C_ROUNDS; ++j)
			STBDS_SIPROUND();
		v0 ^= data;
	}
	data = len << (STBDS_SIZE_T_BITS - 8);
	switch (len - i)
	{
		case 7:
			data |= ((size_t)d[6] << 24) << 24;	 // fall through
		case 6:
			data |= ((size_t)d[5] << 20) << 20;	 // fall through
		case 5:
			data |= ((size_t)d[4] << 16) << 16;	 // fall through
		case 4:
			data |= ((size_t)d[3] << 24);  // fall through
		case 3:
			data |= ((size_t)d[2] << 16);  // fall through
		case 2:
			data |= ((size_t)d[1] << 8);  // fall through
		case 1:
			data |= d[0];  // fall through
		case 0:
			break;
	}
	v3 ^= data;
	for (j = 0; j < STBDS_SIPHASH_C_ROUNDS; ++j)
		STBDS_SIPROUND();
	v0 ^= data;
	v2 ^= 0xff;
	for (j = 0; j < STBDS_SIPHASH_D_ROUNDS; ++j)
		STBDS_SIPROUND();

#ifdef STBDS_SIPHASH_2_4
	return v0 ^ v1 ^ v2 ^ v3;
#else
	return v1 ^ v2 ^ v3;  // slightly stronger since v0^v3 in above cancels out final round operation? I tweeted at the authors of SipHash about this but they
						  // didn't reply
#endif
}

size_t stbds_hash_bytes(void* p, size_t len, size_t seed)
{
#ifdef STBDS_SIPHASH_2_4
	return stbds_siphash_bytes(p, len, seed);
#else
	unsigned char* d = (unsigned char*)p;

	if (len == 4)
	{
		unsigned int hash = d[0] | (d[1] << 8) | (d[2] << 16) | (d[3] << 24);
#if 0
    // HASH32-A  Bob Jenkin's hash function w/o large constants
    hash ^= seed;
    hash -= (hash<<6);
    hash ^= (hash>>17);
    hash -= (hash<<9);
    hash ^= seed;
    hash ^= (hash<<4);
    hash -= (hash<<3);
    hash ^= (hash<<10);
    hash ^= (hash>>15);
#elif 1
		// HASH32-BB  Bob Jenkin's presumably-accidental version of Thomas Wang hash with rotates turned into shifts.
		// Note that converting these back to rotates makes it run a lot slower, presumably due to collisions, so I'm
		// not really sure what's going on.
		hash ^= seed;
		hash = (hash ^ 61) ^ (hash >> 16);
		hash = hash + (hash << 3);
		hash = hash ^ (hash >> 4);
		hash = hash * 0x27d4eb2d;
		hash ^= seed;
		hash = hash ^ (hash >> 15);
#else  // HASH32-C   -  Murmur3
		hash ^= seed;
		hash *= 0xcc9e2d51;
		hash = (hash << 17) | (hash >> 15);
		hash *= 0x1b873593;
		hash ^= seed;
		hash = (hash << 19) | (hash >> 13);
		hash = hash * 5 + 0xe6546b64;
		hash ^= hash >> 16;
		hash *= 0x85ebca6b;
		hash ^= seed;
		hash ^= hash >> 13;
		hash *= 0xc2b2ae35;
		hash ^= hash >> 16;
#endif
		// Following statistics were measured on a Core i7-6700 @ 4.00Ghz, compiled with clang 7.0.1 -O2
		// Note that the larger tables have high variance as they were run fewer times
		//  HASH32-A   //  HASH32-BB  //  HASH32-C
		//    0.10ms   //    0.10ms   //    0.10ms :      2,000 inserts creating 2K table
		//    0.96ms   //    0.95ms   //    0.99ms :     20,000 inserts creating 20K table
		//   14.69ms   //   14.43ms   //   14.97ms :    200,000 inserts creating 200K table
		//  199.99ms   //  195.36ms   //  202.05ms :  2,000,000 inserts creating 2M table
		// 2234.84ms   // 2187.74ms   // 2240.38ms : 20,000,000 inserts creating 20M table
		//   55.68ms   //   53.72ms   //   57.31ms : 500,000 inserts & deletes in 2K table
		//   63.43ms   //   61.99ms   //   65.73ms : 500,000 inserts & deletes in 20K table
		//   80.04ms   //   77.96ms   //   81.83ms : 500,000 inserts & deletes in 200K table
		//  100.42ms   //   97.40ms   //  102.39ms : 500,000 inserts & deletes in 2M table
		//  119.71ms   //  120.59ms   //  121.63ms : 500,000 inserts & deletes in 20M table
		//  185.28ms   //  195.15ms   //  187.74ms : 500,000 inserts & deletes in 200M table
		//   15.58ms   //   14.79ms   //   15.52ms : 200,000 inserts creating 200K table with varying key spacing

		return (((size_t)hash << 16 << 16) | hash) ^ seed;
	}
#if SIZE_MAX == UINT64_MAX
	else if (len == 8)
	{
		size_t hash = d[0] | (d[1] << 8) | (d[2] << 16) | (d[3] << 24);
		hash |= (size_t)(d[4] | (d[5] << 8) | (d[6] << 16) | (d[7] << 24)) << 16 << 16;	 // avoid warning if size_t == 4
		hash ^= seed;
		hash = (~hash) + (hash << 21);
		hash ^= STBDS_ROTATE_RIGHT(hash, 24);
		hash *= 265;
		hash ^= STBDS_ROTATE_RIGHT(hash, 14);
		hash ^= seed;
		hash *= 21;
		hash ^= STBDS_ROTATE_RIGHT(hash, 28);
		hash += (hash << 31);
		hash = (~hash) + (hash << 18);
		return hash;
	}
#endif
	else
	{
		return stbds_siphash_bytes(p, len, seed);
	}
#endif
}
#ifdef _MSC_VER
#pragma warning(pop)
#endif

static int stbds_is_key_equal(void* a, size_t elemsize, void* key, size_t keysize, size_t keyoffset, int mode, size_t i)
{
	if (mode >= STBDS_HM_STRING)
		return 0 == strcmp((char*)key, *(char**)((char*)a + elemsize * i + keyoffset));
	else
		return 0 == memcmp(key, (char*)a + elemsize * i + keyoffset, keysize);
}

#define STBDS_HASH_TO_ARR(x, elemsize) ((char*)(x) - (elemsize))
#define STBDS_ARR_TO_HASH(x, elemsize) ((char*)(x) + (elemsize))

#define stbds_hash_table(a) ((stbds_hash_index*)stbds_header(a)->hash_table)

void stbds_hmfree_func(void* a, size_t elemsize)
{
	if (a == NULL)
		return;
	if (stbds_hash_table(a) != NULL)
	{
		if (stbds_hash_table(a)->string.mode == STBDS_SH_STRDUP)
		{
			size_t i;
			// skip 0th element, which is default
			for (i = 1; i < stbds_header(a)->length; ++i)
				STB_COMMON_FREE(*(char**)((char*)a + elemsize * i));
		}
		stbds_strreset(&stbds_hash_table(a)->string);
	}
	STB_COMMON_FREE(stbds_header(a)->hash_table);
	STB_COMMON_FREE(stbds_header(a));
}

static ptrdiff_t stbds_hm_find_slot(void* a, size_t elemsize, void* key, size_t keysize, size_t keyoffset, int mode)
{
	void* raw_a = STBDS_HASH_TO_ARR(a, elemsize);
	stbds_hash_index* table = stbds_hash_table(raw_a);
	size_t hash = mode >= STBDS_HM_STRING ? stbds_hash_string((char*)key, table->seed) : stbds_hash_bytes(key, keysize, table->seed);
	size_t step = STBDS_BUCKET_LENGTH;
	size_t limit, i;
	size_t pos;
	stbds_hash_bucket* bucket;

	if (hash < 2)
		hash += 2;	// stored hash values are forbidden from being 0, so we can detect empty slots

	pos = stbds_probe_position(hash, table->slot_count, table->slot_count_log2);

	for (;;)
	{
		STBDS_STATS(++stbds_hash_probes);
		bucket = &table->storage[pos >> STBDS_BUCKET_SHIFT];

		// start searching from pos to end of bucket, this should help performance on small hash tables that fit in cache
		for (i = pos & STBDS_BUCKET_MASK; i < STBDS_BUCKET_LENGTH; ++i)
		{
			if (bucket->hash[i] == hash)
			{
				if (stbds_is_key_equal(a, elemsize, key, keysize, keyoffset, mode, bucket->index[i]))
				{
					return (pos & ~STBDS_BUCKET_MASK) + i;
				}
			}
			else if (bucket->hash[i] == STBDS_HASH_EMPTY)
			{
				return -1;
			}
		}

		// search from beginning of bucket to pos
		limit = pos & STBDS_BUCKET_MASK;
		for (i = 0; i < limit; ++i)
		{
			if (bucket->hash[i] == hash)
			{
				if (stbds_is_key_equal(a, elemsize, key, keysize, keyoffset, mode, bucket->index[i]))
				{
					return (pos & ~STBDS_BUCKET_MASK) + i;
				}
			}
			else if (bucket->hash[i] == STBDS_HASH_EMPTY)
			{
				return -1;
			}
		}

		// quadratic probing
		pos += step;
		step += STBDS_BUCKET_LENGTH;
		pos &= (table->slot_count - 1);
	}
	/* NOTREACHED */
}

void* stbds_hmget_key_ts(void* a, size_t elemsize, void* key, size_t keysize, ptrdiff_t* temp, int mode)
{
	size_t keyoffset = 0;
	if (a == NULL)
	{
		// make it non-empty so we can return a temp
		a = stbds_arrgrowf(0, elemsize, 0, 1);
		stbds_header(a)->length += 1;
		memset(a, 0, elemsize);
		*temp = STBDS_INDEX_EMPTY;
		// adjust a to point after the default element
		return STBDS_ARR_TO_HASH(a, elemsize);
	}
	else
	{
		stbds_hash_index* table;
		void* raw_a = STBDS_HASH_TO_ARR(a, elemsize);
		// adjust a to point to the default element
		table = (stbds_hash_index*)stbds_header(raw_a)->hash_table;
		if (table == 0)
		{
			*temp = -1;
		}
		else
		{
			ptrdiff_t slot = stbds_hm_find_slot(a, elemsize, key, keysize, keyoffset, mode);
			if (slot < 0)
			{
				*temp = STBDS_INDEX_EMPTY;
			}
			else
			{
				stbds_hash_bucket* b = &table->storage[slot >> STBDS_BUCKET_SHIFT];
				*temp = b->index[slot & STBDS_BUCKET_MASK];
			}
		}
		return a;
	}
}

void* stbds_hmget_key(void* a, size_t elemsize, void* key, size_t keysize, int mode)
{
	ptrdiff_t temp;
	void* p = stbds_hmget_key_ts(a, elemsize, key, keysize, &temp, mode);
	stbds_temp(STBDS_HASH_TO_ARR(p, elemsize)) = temp;
	return p;
}

void* stbds_hmput_default(void* a, size_t elemsize)
{
	// three cases:
	//   a is NULL <- allocate
	//   a has a hash table but no entries, because of shmode <- grow
	//   a has entries <- do nothing
	if (a == NULL || stbds_header(STBDS_HASH_TO_ARR(a, elemsize))->length == 0)
	{
		a = stbds_arrgrowf(a ? STBDS_HASH_TO_ARR(a, elemsize) : NULL, elemsize, 0, 1);
		stbds_header(a)->length += 1;
		memset(a, 0, elemsize);
		a = STBDS_ARR_TO_HASH(a, elemsize);
	}
	return a;
}

static char* stbds_strdup(char* str);

void* stbds_hmput_key(void* a, size_t elemsize, void* key, size_t keysize, int mode)
{
	size_t keyoffset = 0;
	void* raw_a;
	stbds_hash_index* table;

	if (a == NULL)
	{
		a = stbds_arrgrowf(0, elemsize, 0, 1);
		memset(a, 0, elemsize);
		stbds_header(a)->length += 1;
		// adjust a to point AFTER the default element
		a = STBDS_ARR_TO_HASH(a, elemsize);
	}

	// adjust a to point to the default element
	raw_a = a;
	a = STBDS_HASH_TO_ARR(a, elemsize);

	table = (stbds_hash_index*)stbds_header(a)->hash_table;

	if (table == NULL || table->used_count >= table->used_count_threshold)
	{
		stbds_hash_index* nt;
		size_t slot_count;

		slot_count = (table == NULL) ? STBDS_BUCKET_LENGTH : table->slot_count * 2;
		nt = stbds_make_hash_index(slot_count, table);
		if (table)
			STB_COMMON_FREE(table);
		else
			nt->string.mode = mode >= STBDS_HM_STRING ? STBDS_SH_DEFAULT : 0;
		stbds_header(a)->hash_table = table = nt;
		STBDS_STATS(++stbds_hash_grow);
	}

	// we iterate hash table explicitly because we want to track if we saw a tombstone
	{
		size_t hash = mode >= STBDS_HM_STRING ? stbds_hash_string((char*)key, table->seed) : stbds_hash_bytes(key, keysize, table->seed);
		size_t step = STBDS_BUCKET_LENGTH;
		size_t pos;
		ptrdiff_t tombstone = -1;
		stbds_hash_bucket* bucket;

		// stored hash values are forbidden from being 0, so we can detect empty slots to early out quickly
		if (hash < 2)
			hash += 2;

		pos = stbds_probe_position(hash, table->slot_count, table->slot_count_log2);

		for (;;)
		{
			size_t limit, i;
			STBDS_STATS(++stbds_hash_probes);
			bucket = &table->storage[pos >> STBDS_BUCKET_SHIFT];

			// start searching from pos to end of bucket
			for (i = pos & STBDS_BUCKET_MASK; i < STBDS_BUCKET_LENGTH; ++i)
			{
				if (bucket->hash[i] == hash)
				{
					if (stbds_is_key_equal(raw_a, elemsize, key, keysize, keyoffset, mode, bucket->index[i]))
					{
						stbds_temp(a) = bucket->index[i];
						return STBDS_ARR_TO_HASH(a, elemsize);
					}
				}
				else if (bucket->hash[i] == 0)
				{
					pos = (pos & ~STBDS_BUCKET_MASK) + i;
					goto found_empty_slot;
				}
				else if (tombstone < 0)
				{
					if (bucket->index[i] == STBDS_INDEX_DELETED)
						tombstone = (ptrdiff_t)((pos & ~STBDS_BUCKET_MASK) + i);
				}
			}

			// search from beginning of bucket to pos
			limit = pos & STBDS_BUCKET_MASK;
			for (i = 0; i < limit; ++i)
			{
				if (bucket->hash[i] == hash)
				{
					if (stbds_is_key_equal(raw_a, elemsize, key, keysize, keyoffset, mode, bucket->index[i]))
					{
						stbds_temp(a) = bucket->index[i];
						return STBDS_ARR_TO_HASH(a, elemsize);
					}
				}
				else if (bucket->hash[i] == 0)
				{
					pos = (pos & ~STBDS_BUCKET_MASK) + i;
					goto found_empty_slot;
				}
				else if (tombstone < 0)
				{
					if (bucket->index[i] == STBDS_INDEX_DELETED)
						tombstone = (ptrdiff_t)((pos & ~STBDS_BUCKET_MASK) + i);
				}
			}

			// quadratic probing
			pos += step;
			step += STBDS_BUCKET_LENGTH;
			pos &= (table->slot_count - 1);
		}
	found_empty_slot:
		if (tombstone >= 0)
		{
			pos = tombstone;
			--table->tombstone_count;
		}
		++table->used_count;

		{
			ptrdiff_t j = (ptrdiff_t)stbds_arrlen(a);
			// we want to do stbds_arraddn(1), but we can't use the macros since we don't have something of the right type
			if ((size_t)j + 1 > stbds_arrcap(a))
				*(void**)&a = stbds_arrgrowf(a, elemsize, 1, 0);
			raw_a = STBDS_ARR_TO_HASH(a, elemsize);

			STBDS_ASSERT((size_t)j + 1 <= stbds_arrcap(a));
			stbds_header(a)->length = j + 1;
			bucket = &table->storage[pos >> STBDS_BUCKET_SHIFT];
			bucket->hash[pos & STBDS_BUCKET_MASK] = hash;
			bucket->index[pos & STBDS_BUCKET_MASK] = j - 1;
			stbds_temp(a) = j - 1;

			switch (table->string.mode)
			{
				case STBDS_SH_STRDUP:
					stbds_temp_key(a) = *(char**)((char*)a + elemsize * j) = stbds_strdup((char*)key);
					break;
				case STBDS_SH_ARENA:
					stbds_temp_key(a) = *(char**)((char*)a + elemsize * j) = stbds_stralloc(&table->string, (char*)key);
					break;
				case STBDS_SH_DEFAULT:
					stbds_temp_key(a) = *(char**)((char*)a + elemsize * j) = (char*)key;
					break;
				default:
					memcpy((char*)a + elemsize * j, key, keysize);
					break;
			}
		}
		return STBDS_ARR_TO_HASH(a, elemsize);
	}
}

void* stbds_shmode_func(size_t elemsize, int mode)
{
	void* a = stbds_arrgrowf(0, elemsize, 0, 1);
	stbds_hash_index* h;
	memset(a, 0, elemsize);
	stbds_header(a)->length = 1;
	stbds_header(a)->hash_table = h = (stbds_hash_index*)stbds_make_hash_index(STBDS_BUCKET_LENGTH, NULL);
	h->string.mode = (unsigned char)mode;
	return STBDS_ARR_TO_HASH(a, elemsize);
}

void* stbds_hmdel_key(void* a, size_t elemsize, void* key, size_t keysize, size_t keyoffset, int mode)
{
	if (a == NULL)
	{
		return 0;
	}
	else
	{
		stbds_hash_index* table;
		void* raw_a = STBDS_HASH_TO_ARR(a, elemsize);
		table = (stbds_hash_index*)stbds_header(raw_a)->hash_table;
		stbds_temp(raw_a) = 0;
		if (table == 0)
		{
			return a;
		}
		else
		{
			ptrdiff_t slot;
			slot = stbds_hm_find_slot(a, elemsize, key, keysize, keyoffset, mode);
			if (slot < 0)
				return a;
			else
			{
				stbds_hash_bucket* b = &table->storage[slot >> STBDS_BUCKET_SHIFT];
				int i = slot & STBDS_BUCKET_MASK;
				ptrdiff_t old_index = b->index[i];
				ptrdiff_t final_index = (ptrdiff_t)stbds_arrlen(raw_a) - 1 - 1;	 // minus one for the raw_a vs a, and minus one for 'last'
				STBDS_ASSERT(slot < (ptrdiff_t)table->slot_count);
				--table->used_count;
				++table->tombstone_count;
				stbds_temp(raw_a) = 1;
				STBDS_ASSERT(table->used_count >= 0);
				// STBDS_ASSERT(table->tombstone_count < table->slot_count/4);
				b->hash[i] = STBDS_HASH_DELETED;
				b->index[i] = STBDS_INDEX_DELETED;

				if (mode == STBDS_HM_STRING && table->string.mode == STBDS_SH_STRDUP)
					STB_COMMON_FREE(*(char**)((char*)a + elemsize * old_index));

				// if indices are the same, memcpy is a no-op, but back-pointer-fixup will fail, so skip
				if (old_index != final_index)
				{
					// swap delete
					memmove((char*)a + elemsize * old_index, (char*)a + elemsize * final_index, elemsize);

					// now find the slot for the last element
					if (mode == STBDS_HM_STRING)
						slot = stbds_hm_find_slot(a, elemsize, *(char**)((char*)a + elemsize * old_index + keyoffset), keysize, keyoffset, mode);
					else
						slot = stbds_hm_find_slot(a, elemsize, (char*)a + elemsize * old_index + keyoffset, keysize, keyoffset, mode);
					STBDS_ASSERT(slot >= 0);
					b = &table->storage[slot >> STBDS_BUCKET_SHIFT];
					i = slot & STBDS_BUCKET_MASK;
					STBDS_ASSERT(b->index[i] == final_index);
					b->index[i] = old_index;
				}
				stbds_header(raw_a)->length -= 1;

				if (table->used_count < table->used_count_shrink_threshold && table->slot_count > STBDS_BUCKET_LENGTH)
				{
					stbds_header(raw_a)->hash_table = stbds_make_hash_index(table->slot_count >> 1, table);
					STB_COMMON_FREE(table);
					STBDS_STATS(++stbds_hash_shrink);
				}
				else if (table->tombstone_count > table->tombstone_count_threshold)
				{
					stbds_header(raw_a)->hash_table = stbds_make_hash_index(table->slot_count, table);
					STB_COMMON_FREE(table);
					STBDS_STATS(++stbds_hash_rebuild);
				}

				return a;
			}
		}
	}
	/* NOTREACHED */
}

static char* stbds_strdup(char* str)
{
	// to keep replaceable allocator simple, we don't want to use strdup.
	// rolling our own also avoids problem of strdup vs _strdup
	size_t len = strlen(str) + 1;
	char* p = (char*)STB_COMMON_REALLOC(0, len);
	memmove(p, str, len);
	return p;
}

#ifndef STBDS_STRING_ARENA_BLOCKSIZE_MIN
#define STBDS_STRING_ARENA_BLOCKSIZE_MIN 512u
#endif
#ifndef STBDS_STRING_ARENA_BLOCKSIZE_MAX
#define STBDS_STRING_ARENA_BLOCKSIZE_MAX (1u << 20)
#endif

char* stbds_stralloc(stbds_string_arena* a, char* str)
{
	char* p;
	size_t len = strlen(str) + 1;
	if (len > a->remaining)
	{
		// compute the next blocksize
		size_t blocksize = a->block;

		// size is 512, 512, 1024, 1024, 2048, 2048, 4096, 4096, etc., so that
		// there are log(SIZE) allocations to free when we destroy the table
		blocksize = (size_t)(STBDS_STRING_ARENA_BLOCKSIZE_MIN) << (blocksize >> 1);

		// if size is under 1M, advance to next blocktype
		if (blocksize < (size_t)(STBDS_STRING_ARENA_BLOCKSIZE_MAX))
			++a->block;

		if (len > blocksize)
		{
			// if string is larger than blocksize, then just allocate the full size.
			// note that we still advance string_block so block size will continue
			// increasing, so e.g. if somebody only calls this with 1000-long strings,
			// eventually the arena will start doubling and handling those as well
			stbds_string_block* sb = (stbds_string_block*)STB_COMMON_REALLOC(0, sizeof(*sb) + len);
			if (!sb)
				return NULL;
			memmove(sb->storage, str, len);
			if (a->storage)
			{
				// insert it after the first element, so that we don't waste the space there
				sb->next = a->storage->next;
				a->storage->next = sb;
			}
			else
			{
				sb->next = 0;
				a->storage = sb;
				a->remaining = 0;  // this is redundant, but good for clarity
			}
			return sb->storage;
		}
		else
		{
			stbds_string_block* sb = (stbds_string_block*)STB_COMMON_REALLOC(0, sizeof(*sb) + blocksize);
			if (!sb)
				return NULL;
			sb->next = a->storage;
			a->storage = sb;
			a->remaining = blocksize;
		}
	}

	STBDS_ASSERT(len <= a->remaining);
	p = a->storage->storage + a->remaining - len;
	a->remaining -= len;
	memmove(p, str, len);
	return p;
}

void stbds_strreset(stbds_string_arena* a)
{
	stbds_string_block *x, *y;
	x = a->storage;
	while (x)
	{
		y = x->next;
		STB_COMMON_FREE(x);
		x = y;
	}
	memset(a, 0, sizeof(*a));
}

//////////////////////////////////////////////////////////////////////////////
//
//   UNIT TESTS
//

#ifdef STBDS_UNIT_TESTS
#include <stdio.h>
#ifdef STBDS_ASSERT_WAS_UNDEFINED
#undef STBDS_ASSERT
#endif
#ifndef STBDS_ASSERT
#define STBDS_ASSERT assert
#include <assert.h>
#endif

typedef struct
{
	int key, b, c, d;
} stbds_struct;
typedef struct
{
	int key[2], b, c, d;
} stbds_struct2;

static char buffer[256];
char* strkey(int n)
{
#if defined(_WIN32) && defined(__STDC_WANT_SECURE_LIB__)
	sprintf_s(buffer, sizeof(buffer), "test_%d", n);
#else
	sprintf(buffer, "test_%d", n);
#endif
	return buffer;
}

void stbds_unit_tests(void)
{
#if defined(_MSC_VER) && _MSC_VER <= 1200 && defined(__cplusplus)
	// VC6 C++ doesn't like the template<> trick on unnamed structures, so do nothing!
	STBDS_ASSERT(0);
#else
	const int testsize = 100000;
	const int testsize2 = testsize / 20;
	int* arr = NULL;
	struct
	{
		int key;
		int value;
	}* intmap = NULL;
	struct
	{
		char* key;
		int value;
	}* strmap = NULL, s;
	struct
	{
		stbds_struct key;
		int value;
	}* map = NULL;
	stbds_struct* map2 = NULL;
	stbds_struct2* map3 = NULL;
	stbds_string_arena sa = {0};
	int key3[2] = {1, 2};
	ptrdiff_t temp;

	int i, j;

	STBDS_ASSERT(arrlen(arr) == 0);
	for (i = 0; i < 20000; i += 50)
	{
		for (j = 0; j < i; ++j)
			arrpush(arr, j);
		arrfree(arr);
	}

	for (i = 0; i < 4; ++i)
	{
		arrpush(arr, 1);
		arrpush(arr, 2);
		arrpush(arr, 3);
		arrpush(arr, 4);
		arrdel(arr, i);
		arrfree(arr);
		arrpush(arr, 1);
		arrpush(arr, 2);
		arrpush(arr, 3);
		arrpush(arr, 4);
		arrdelswap(arr, i);
		arrfree(arr);
	}

	for (i = 0; i < 5; ++i)
	{
		arrpush(arr, 1);
		arrpush(arr, 2);
		arrpush(arr, 3);
		arrpush(arr, 4);
		stbds_arrins(arr, i, 5);
		STBDS_ASSERT(arr[i] == 5);
		if (i < 4)
			STBDS_ASSERT(arr[4] == 4);
		arrfree(arr);
	}

	i = 1;
	STBDS_ASSERT(hmgeti(intmap, i) == -1);
	hmdefault(intmap, -2);
	STBDS_ASSERT(hmgeti(intmap, i) == -1);
	STBDS_ASSERT(hmget(intmap, i) == -2);
	for (i = 0; i < testsize; i += 2)
		hmput(intmap, i, i * 5);
	for (i = 0; i < testsize; i += 1)
	{
		if (i & 1)
			STBDS_ASSERT(hmget(intmap, i) == -2);
		else
			STBDS_ASSERT(hmget(intmap, i) == i * 5);
		if (i & 1)
			STBDS_ASSERT(hmget_ts(intmap, i, temp) == -2);
		else
			STBDS_ASSERT(hmget_ts(intmap, i, temp) == i * 5);
	}
	for (i = 0; i < testsize; i += 2)
		hmput(intmap, i, i * 3);
	for (i = 0; i < testsize; i += 1)
		if (i & 1)
			STBDS_ASSERT(hmget(intmap, i) == -2);
		else
			STBDS_ASSERT(hmget(intmap, i) == i * 3);
	for (i = 2; i < testsize; i += 4)
		hmdel(intmap, i);  // delete half the entries
	for (i = 0; i < testsize; i += 1)
		if (i & 3)
			STBDS_ASSERT(hmget(intmap, i) == -2);
		else
			STBDS_ASSERT(hmget(intmap, i) == i * 3);
	for (i = 0; i < testsize; i += 1)
		hmdel(intmap, i);  // delete the rest of the entries
	for (i = 0; i < testsize; i += 1)
		STBDS_ASSERT(hmget(intmap, i) == -2);
	hmfree(intmap);
	for (i = 0; i < testsize; i += 2)
		hmput(intmap, i, i * 3);
	hmfree(intmap);

#if defined(__clang__) || defined(__GNUC__)
#ifndef __cplusplus
	intmap = NULL;
	hmput(intmap, 15, 7);
	hmput(intmap, 11, 3);
	hmput(intmap, 9, 5);
	STBDS_ASSERT(hmget(intmap, 9) == 5);
	STBDS_ASSERT(hmget(intmap, 11) == 3);
	STBDS_ASSERT(hmget(intmap, 15) == 7);
#endif
#endif

	for (i = 0; i < testsize; ++i)
		stralloc(&sa, strkey(i));
	strreset(&sa);

	{
		s.key = "a", s.value = 1;
		shputs(strmap, s);
		STBDS_ASSERT(*strmap[0].key == 'a');
		STBDS_ASSERT(strmap[0].key == s.key);
		STBDS_ASSERT(strmap[0].value == s.value);
		shfree(strmap);
	}

	{
		s.key = "a", s.value = 1;
		sh_new_strdup(strmap);
		shputs(strmap, s);
		STBDS_ASSERT(*strmap[0].key == 'a');
		STBDS_ASSERT(strmap[0].key != s.key);
		STBDS_ASSERT(strmap[0].value == s.value);
		shfree(strmap);
	}

	{
		s.key = "a", s.value = 1;
		sh_new_arena(strmap);
		shputs(strmap, s);
		STBDS_ASSERT(*strmap[0].key == 'a');
		STBDS_ASSERT(strmap[0].key != s.key);
		STBDS_ASSERT(strmap[0].value == s.value);
		shfree(strmap);
	}

	for (j = 0; j < 2; ++j)
	{
		STBDS_ASSERT(shgeti(strmap, "foo") == -1);
		if (j == 0)
			sh_new_strdup(strmap);
		else
			sh_new_arena(strmap);
		STBDS_ASSERT(shgeti(strmap, "foo") == -1);
		shdefault(strmap, -2);
		STBDS_ASSERT(shgeti(strmap, "foo") == -1);
		for (i = 0; i < testsize; i += 2)
			shput(strmap, strkey(i), i * 3);
		for (i = 0; i < testsize; i += 1)
			if (i & 1)
				STBDS_ASSERT(shget(strmap, strkey(i)) == -2);
			else
				STBDS_ASSERT(shget(strmap, strkey(i)) == i * 3);
		for (i = 2; i < testsize; i += 4)
			shdel(strmap, strkey(i));  // delete half the entries
		for (i = 0; i < testsize; i += 1)
			if (i & 3)
				STBDS_ASSERT(shget(strmap, strkey(i)) == -2);
			else
				STBDS_ASSERT(shget(strmap, strkey(i)) == i * 3);
		for (i = 0; i < testsize; i += 1)
			shdel(strmap, strkey(i));  // delete the rest of the entries
		for (i = 0; i < testsize; i += 1)
			STBDS_ASSERT(shget(strmap, strkey(i)) == -2);
		shfree(strmap);
	}

	{
		struct
		{
			char* key;
			char value;
		}* hash = NULL;
		char name[4] = "jen";
		shput(hash, "bob", 'h');
		shput(hash, "sally", 'e');
		shput(hash, "fred", 'l');
		shput(hash, "jen", 'x');
		shput(hash, "doug", 'o');

		shput(hash, name, 'l');
		shfree(hash);
	}

	for (i = 0; i < testsize; i += 2)
	{
		stbds_struct s = {i, i * 2, i * 3, i * 4};
		hmput(map, s, i * 5);
	}

	for (i = 0; i < testsize; i += 1)
	{
		stbds_struct s = {i, i * 2, i * 3, i * 4};
		stbds_struct t = {i, i * 2, i * 3 + 1, i * 4};
		if (i & 1)
			STBDS_ASSERT(hmget(map, s) == 0);
		else
			STBDS_ASSERT(hmget(map, s) == i * 5);
		if (i & 1)
			STBDS_ASSERT(hmget_ts(map, s, temp) == 0);
		else
			STBDS_ASSERT(hmget_ts(map, s, temp) == i * 5);
		// STBDS_ASSERT(hmget(map, t.key) == 0);
	}

	for (i = 0; i < testsize; i += 2)
	{
		stbds_struct s = {i, i * 2, i * 3, i * 4};
		hmputs(map2, s);
	}
	hmfree(map);

	for (i = 0; i < testsize; i += 1)
	{
		stbds_struct s = {i, i * 2, i * 3, i * 4};
		stbds_struct t = {i, i * 2, i * 3 + 1, i * 4};
		if (i & 1)
			STBDS_ASSERT(hmgets(map2, s.key).d == 0);
		else
			STBDS_ASSERT(hmgets(map2, s.key).d == i * 4);
		// STBDS_ASSERT(hmgetp(map2, t.key) == 0);
	}
	hmfree(map2);

	for (i = 0; i < testsize; i += 2)
	{
		stbds_struct2 s = {{i, i * 2}, i * 3, i * 4, i * 5};
		hmputs(map3, s);
	}
	for (i = 0; i < testsize; i += 1)
	{
		stbds_struct2 s = {{i, i * 2}, i * 3, i * 4, i * 5};
		stbds_struct2 t = {{i, i * 2}, i * 3 + 1, i * 4, i * 5};
		if (i & 1)
			STBDS_ASSERT(hmgets(map3, s.key).d == 0);
		else
			STBDS_ASSERT(hmgets(map3, s.key).d == i * 5);
		// STBDS_ASSERT(hmgetp(map3, t.key) == 0);
	}
#endif
}
#endif