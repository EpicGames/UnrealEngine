// Copyright Epic Games, Inc. All Rights Reserved.

/* stb_ds.h - v0.65 - public domain data structures - Sean Barrett 2019

   This is a single-header-file library that provides easy-to-use
   dynamic arrays and hash tables for C (also works in C++).

   For a gentle introduction:
	  http://nothings.org/stb_ds

TABLE OF CONTENTS

  Table of Contents
  Compile-time options
  License
  Documentation
  Notes
  Notes - Dynamic arrays
  Notes - Hash maps
  Credits

COMPILE-TIME OPTIONS

  #define STBDS_NO_SHORT_NAMES

	 This flag needs to be set globally.

	 By default stb_ds exposes shorter function names that are not qualified
	 with the "stbds_" prefix. If these names conflict with the names in your
	 code, define this flag.

  #define STBDS_SIPHASH_2_4

	 By default stb_ds.h hashes using a weaker variant of SipHash and a custom hash for
	 4- and 8-byte keys. On 64-bit platforms, you can define the above flag to force
	 stb_ds.h to use specification-compliant SipHash-2-4 for all keys. Doing so makes
	 hash table insertion about 20% slower on 4- and 8-byte keys, 5% slower on
	 64-byte keys, and 10% slower on 256-byte keys on my test computer.

  #define STBDS_UNIT_TESTS

	 Defines a function stbds_unit_tests() that checks the functioning of the data structures.

  Note that on older versions of gcc (e.g. 5.x.x) you may need to build with '-std=c++0x'
	 (or equivalentally '-std=c++11') when using anonymous structures as seen on the web
	 page or in STBDS_UNIT_TESTS.

LICENSE

  Placed in the public domain and also MIT licensed.
  See end of file for detailed license information.

DOCUMENTATION

  Dynamic Arrays

	Non-function interface:

	  Declare an empty dynamic array of type T
		T* foo = NULL;

	  Access the i'th item of a dynamic array 'foo' of type T, T* foo:
		foo[i]

	Functions (actually macros)

	  arrfree:
		void arrfree(T*);
		  Frees the array.

	  arrlen:
		ptrdiff_t arrlen(T*);
		  Returns the number of elements in the array.

	  arrlenu:
		size_t arrlenu(T*);
		  Returns the number of elements in the array as an unsigned type.

	  arrpop:
		T arrpop(T* a)
		  Removes the final element of the array and returns it.

	  arrput:
		T arrput(T* a, T b);
		  Appends the item b to the end of array a. Returns b.

	  arrins:
		T arrins(T* a, int p, T b);
		  Inserts the item b into the middle of array a, into a[p],
		  moving the rest of the array over. Returns b.

	  arrinsn:
		void arrins(T* a, int p, int n);
		  Inserts n uninitialized items into array a starting at a[p],
		  moving the rest of the array over.

	  arraddnptr:
		T* arraddnptr(T* a, int n)
		  Appends n uninitialized items onto array at the end.
		  Returns a pointer to the first uninitialized item added.

	  arraddnindex:
		size_t arraddnindex(T* a, int n)
		  Appends n uninitialized items onto array at the end.
		  Returns the index of the first uninitialized item added.

	  arrdel:
		void arrdel(T* a, int p);
		  Deletes the element at a[p], moving the rest of the array over.

	  arrdeln:
		void arrdel(T* a, int p, int n);
		  Deletes n elements starting at a[p], moving the rest of the array over.

	  arrdelswap:
		void arrdelswap(T* a, int p);
		  Deletes the element at a[p], replacing it with the element from
		  the end of the array. O(1) performance.

	  arrsetlen:
		void arrsetlen(T* a, int n);
		  Changes the length of the array to n. Allocates uninitialized
		  slots at the end if necessary.

	  arrsetcap:
		size_t arrsetcap(T* a, int n);
		  Sets the length of allocated storage to at least n. It will not
		  change the length of the array.

	  arrcap:
		size_t arrcap(T* a);
		  Returns the number of total elements the array can contain without
		  needing to be reallocated.

  Hash maps & String hash maps

	Given T is a structure type: struct { TK key; TV value; }. Note that some
	functions do not require TV value and can have other fields. For string
	hash maps, TK must be 'char *'.

	Special interface:


	  stbds_hash_string:
		size_t stbds_hash_string(char *str, size_t seed);
		  Returns a hash value for a string.

	  stbds_hash_bytes:
		size_t stbds_hash_bytes(void *p, size_t len, size_t seed);
		  These functions hash an arbitrary number of bytes. The function
		  uses a custom hash for 4- and 8-byte data, and a weakened version
		  of SipHash for everything else. On 64-bit platforms you can get
		  specification-compliant SipHash-2-4 on all data by defining
		  STBDS_SIPHASH_2_4, at a significant cost in speed.

	Non-function interface:

	  Declare an empty hash map of type T
		T* foo = NULL;

	  Access the i'th entry in a hash table T* foo:
		foo[i]

	Function interface (actually macros):

	  hmfree
	  shfree
		void hmfree(T*);
		void shfree(T*);
		  Frees the hashmap and sets the pointer to NULL.

	  hmlen
	  shlen
		ptrdiff_t hmlen(T*)
		ptrdiff_t shlen(T*)
		  Returns the number of elements in the hashmap.

	  hmlenu
	  shlenu
		size_t hmlenu(T*)
		size_t shlenu(T*)
		  Returns the number of elements in the hashmap.

	  hmgeti
	  shgeti
	  hmgeti_ts
		ptrdiff_t hmgeti(T*, TK key)
		ptrdiff_t shgeti(T*, char* key)
		ptrdiff_t hmgeti_ts(T*, TK key, ptrdiff_t tempvar)
		  Returns the index in the hashmap which has the key 'key', or -1
		  if the key is not present.

	  hmget
	  hmget_ts
	  shget
		TV hmget(T*, TK key)
		TV shget(T*, char* key)
		TV hmget_ts(T*, TK key, ptrdiff_t tempvar)
		  Returns the value corresponding to 'key' in the hashmap.
		  The structure must have a 'value' field

	  hmgets
	  shgets
		T hmgets(T*, TK key)
		T shgets(T*, char* key)
		  Returns the structure corresponding to 'key' in the hashmap.

	  hmgetp
	  shgetp
	  hmgetp_ts
	  hmgetp_null
	  shgetp_null
		T* hmgetp(T*, TK key)
		T* shgetp(T*, char* key)
		T* hmgetp_ts(T*, TK key, ptrdiff_t tempvar)
		T* hmgetp_null(T*, TK key)
		T* shgetp_null(T*, char *key)
		  Returns a pointer to the structure corresponding to 'key' in
		  the hashmap. Functions ending in "_null" return NULL if the key
		  is not present in the hashmap; the others return a pointer to a
		  structure holding the default value (but not the searched-for key).

	  hmdefault
	  shdefault
		TV hmdefault(T*, TV value)
		TV shdefault(T*, TV value)
		  Sets the default value for the hashmap, the value which will be
		  returned by hmget/shget if the key is not present.

	  hmdefaults
	  shdefaults
		TV hmdefaults(T*, T item)
		TV shdefaults(T*, T item)
		  Sets the default struct for the hashmap, the contents which will be
		  returned by hmgets/shgets if the key is not present.

	  hmput
	  shput
		TV hmput(T*, TK key, TV value)
		TV shput(T*, char* key, TV value)
		  Inserts a <key,value> pair into the hashmap. If the key is already
		  present in the hashmap, updates its value.

	  hmputs
	  shputs
		T hmputs(T*, T item)
		T shputs(T*, T item)
		  Inserts a struct with T.key into the hashmap. If the struct is already
		  present in the hashmap, updates it.

	  hmdel
	  shdel
		int hmdel(T*, TK key)
		int shdel(T*, char* key)
		  If 'key' is in the hashmap, deletes its entry and returns 1.
		  Otherwise returns 0.

	Function interface (actually macros) for strings only:

	  sh_new_strdup
		void sh_new_strdup(T*);
		  Overwrites the existing pointer with a newly allocated
		  string hashmap which will automatically allocate and free
		  each string key using realloc/free

	  sh_new_arena
		void sh_new_arena(T*);
		  Overwrites the existing pointer with a newly allocated
		  string hashmap which will automatically allocate each string
		  key to a string arena. Every string key ever used by this
		  hash table remains in the arena until the arena is freed.
		  Additionally, any key which is deleted and reinserted will
		  be allocated multiple times in the string arena.

NOTES

  * These data structures are realloc'd when they grow, and the macro
	"functions" write to the provided pointer. This means: (a) the pointer
	must be an lvalue, and (b) the pointer to the data structure is not
	stable, and you must maintain it the same as you would a realloc'd
	pointer. For example, if you pass a pointer to a dynamic array to a
	function which updates it, the function must return back the new
	pointer to the caller. This is the price of trying to do this in C.

  * The following are the only functions that are thread-safe on a single data
	structure, i.e. can be run in multiple threads simultaneously on the same
	data structure
		hmlen        shlen
		hmlenu       shlenu
		hmget_ts     shget_ts
		hmgeti_ts    shgeti_ts
		hmgets_ts    shgets_ts

  * You iterate over the contents of a dynamic array and a hashmap in exactly
	the same way, using arrlen/hmlen/shlen:

	  for (i=0; i < arrlen(foo); ++i)
		 ... foo[i] ...

  * All operations except arrins/arrdel are O(1) amortized, but individual
	operations can be slow, so these data structures may not be suitable
	for real time use. Dynamic arrays double in capacity as needed, so
	elements are copied an average of once. Hash tables double/halve
	their size as needed, with appropriate hysteresis to maintain O(1)
	performance.

NOTES - DYNAMIC ARRAY

  * If you know how long a dynamic array is going to be in advance, you can avoid
	extra memory allocations by using arrsetlen to allocate it to that length in
	advance and use foo[n] while filling it out, or arrsetcap to allocate the memory
	for that length and use arrput/arrpush as normal.

  * Unlike some other versions of the dynamic array, this version should
	be safe to use with strict-aliasing optimizations.

NOTES - HASH MAP

  * For compilers other than GCC and clang (e.g. Visual Studio), for hmput/hmget/hmdel
	and variants, the key must be an lvalue (so the macro can take the address of it).
	Extensions are used that eliminate this requirement if you're using C99 and later
	in GCC or clang, or if you're using C++ in GCC. But note that this can make your
	code less portable.

  * To test for presence of a key in a hashmap, just do 'hmgeti(foo,key) >= 0'.

  * The iteration order of your data in the hashmap is determined solely by the
	order of insertions and deletions. In particular, if you never delete, new
	keys are always added at the end of the array. This will be consistent
	across all platforms and versions of the library. However, you should not
	attempt to serialize the internal hash table, as the hash is not consistent
	between different platforms, and may change with future versions of the library.

  * Use sh_new_arena() for string hashmaps that you never delete from. Initialize
	with NULL if you're managing the memory for your strings, or your strings are
	never freed (at least until the hashmap is freed). Otherwise, use sh_new_strdup().
	@TODO: make an arena variant that garbage collects the strings with a trivial
	copy collector into a new arena whenever the table shrinks / rebuilds. Since
	current arena recommendation is to only use arena if it never deletes, then
	this can just replace current arena implementation.

  * If adversarial input is a serious concern and you're on a 64-bit platform,
	enable STBDS_SIPHASH_2_4 (see the 'Compile-time options' section), and ensure
	values returned by STB_LCG_NEXT function implementation are strong random numbers.

  * The default value for the hash table is stored in foo[-1], so if you
	use code like 'hmget(T,k)->value = 5' you can accidentally overwrite
	the value stored by hmdefault if 'k' is not present.

CREDITS

  Sean Barrett -- library, idea for dynamic array API/implementation
  Per Vognsen  -- idea for hash table API/implementation
  Rafael Sachetto -- arrpop()
  github:HeroicKatora -- arraddn() reworking

  Bugfixes:
	Andy Durdin
	Shane Liesegang
	Vinh Truong
	Andreas Molzer
	github:hashitaku
	github:srdjanstipic
*/

#pragma once

#ifdef STBDS_UNIT_TESTS
#define _CRT_SECURE_NO_WARNINGS
#endif

#include <stddef.h>
#include <stdint.h>
#include <string.h>

#include "stb_common.h"

#ifndef STBDS_NO_SHORT_NAMES
#define arrlen stbds_arrlen
#define arrlennonull stbds_arrlennonull
#define arrlenu stbds_arrlenu
#define arrput stbds_arrput
#define arrpush stbds_arrput
#define arrpop stbds_arrpop
#define arrfree stbds_arrfree
#define arraddn stbds_arraddn  // deprecated, use one of the following instead:
#define arraddnptr stbds_arraddnptr
#define arraddnindex stbds_arraddnindex
#define arrsetlen stbds_arrsetlen
#define arrsetlennocap stbds_arrsetlennocap
#define arrinitlen stbds_arrinitlen
#define arrlast stbds_arrlast
#define arrins stbds_arrins
#define arrinsn stbds_arrinsn
#define arrdel stbds_arrdel
#define arrdeln stbds_arrdeln
#define arrdelswap stbds_arrdelswap
#define arrcap stbds_arrcap
#define arrcapnonull stbds_arrcapnonull
#define arrsetcap stbds_arrsetcap

#define hmput stbds_hmput
#define hmputs stbds_hmputs
#define hmget stbds_hmget
#define hmget_ts stbds_hmget_ts
#define hmgets stbds_hmgets
#define hmgetp stbds_hmgetp
#define hmgetp_ts stbds_hmgetp_ts
#define hmgetp_null stbds_hmgetp_null
#define hmgeti stbds_hmgeti
#define hmgeti_ts stbds_hmgeti_ts
#define hmdel stbds_hmdel
#define hmlen stbds_hmlen
#define hmlenu stbds_hmlenu
#define hmfree stbds_hmfree
#define hmdefault stbds_hmdefault
#define hmdefaults stbds_hmdefaults

#define shput stbds_shput
#define shputi stbds_shputi
#define shputs stbds_shputs
#define shget stbds_shget
#define shgeti stbds_shgeti
#define shgets stbds_shgets
#define shgetp stbds_shgetp
#define shgetp_null stbds_shgetp_null
#define shdel stbds_shdel
#define shlen stbds_shlen
#define shlenu stbds_shlenu
#define shfree stbds_shfree
#define shdefault stbds_shdefault
#define shdefaults stbds_shdefaults
#define sh_new_arena stbds_sh_new_arena
#define sh_new_strdup stbds_sh_new_strdup

#define stralloc stbds_stralloc
#define strreset stbds_strreset
#endif

#ifdef _MSC_VER
#define STBDS_NOTUSED(v) (void)(v)
#else
#define STBDS_NOTUSED(v) (void)sizeof(v)
#endif

#ifdef __cplusplus
extern "C"
{
#endif

	// these are the hash functions used internally if you want to test them or use them for other purposes
	extern size_t stbds_hash_bytes(void* p, size_t len, size_t seed);
	extern size_t stbds_hash_string(char* str, size_t seed);

	// this is a simple string arena allocator, initialize with e.g. 'stbds_string_arena my_arena={0}'.
	typedef struct stbds_string_arena stbds_string_arena;
	extern char* stbds_stralloc(stbds_string_arena* a, char* str);
	extern void stbds_strreset(stbds_string_arena* a);

	// have to #define STBDS_UNIT_TESTS to call this
	extern void stbds_unit_tests(void);

	///////////////
	//
	// Everything below here is implementation details
	//

	extern void* stbds_arrinlinef(size_t* buf, size_t elemsize, size_t elemcount);
	extern void* stbds_arrinline_suballocf(void* a, size_t min_capacity);
	extern void* stbds_arrgrowf(void* a, size_t elemsize, size_t addlen, size_t min_cap);
	extern void stbds_hmfree_func(void* p, size_t elemsize);
	extern void* stbds_hmget_key(void* a, size_t elemsize, void* key, size_t keysize, int mode);
	extern void* stbds_hmget_key_ts(void* a, size_t elemsize, void* key, size_t keysize, ptrdiff_t* temp, int mode);
	extern void* stbds_hmput_default(void* a, size_t elemsize);
	extern void* stbds_hmput_key(void* a, size_t elemsize, void* key, size_t keysize, int mode);
	extern void* stbds_hmdel_key(void* a, size_t elemsize, void* key, size_t keysize, size_t keyoffset, int mode);
	extern void* stbds_shmode_func(size_t elemsize, int mode);

#ifdef __cplusplus
}
#endif

#if defined(__GNUC__) || defined(__clang__)
#define STBDS_HAS_TYPEOF
#ifdef __cplusplus
//#define STBDS_HAS_LITERAL_ARRAY  // this is currently broken for clang
#endif
#endif

#if !defined(__cplusplus)
#if defined(__STDC_VERSION__) && __STDC_VERSION__ >= 199901L
#define STBDS_HAS_LITERAL_ARRAY
#endif
#endif

// this macro takes the address of the argument, but on gcc/clang can accept rvalues
#if defined(STBDS_HAS_LITERAL_ARRAY) && defined(STBDS_HAS_TYPEOF)
#if __clang__
#define STBDS_ADDRESSOF(typevar, value) ((__typeof__(typevar)[1]){value})  // literal array decays to pointer to value
#else
#define STBDS_ADDRESSOF(typevar, value) ((typeof(typevar)[1]){value})  // literal array decays to pointer to value
#endif
#else
#define STBDS_ADDRESSOF(typevar, value) &(value)
#endif

#define STBDS_OFFSETOF(var, field) ((char*)&(var)->field - (char*)(var))

#define stbds_header(t) ((stbds_array_header*)(t)-1)
#define stbds_temp(t) stbds_header(t)->temp
#define stbds_temp_key(t) (*(char**)stbds_header(t)->hash_table)

// Macro to reserve inline stack memory storage for an array
#define stbds_arrinline(a, a_type, n) \
	size_t __buf ## a[(sizeof(stbds_array_header) + n * sizeof(a_type) + sizeof(size_t) - 1) / sizeof(size_t)];		\
	((stbds_array_header*)(&__buf ## a[0]))->length = 0;															\
	((stbds_array_header*)(&__buf ## a[0]))->capacity = n;															\
	((stbds_array_header*)(&__buf ## a[0]))->hash_table = 0;														\
	((stbds_array_header*)(&__buf ## a[0]))->temp = 0;																\
	((stbds_array_header*)(&__buf ## a[0]))->elemsize = sizeof(a_type);												\
	((stbds_array_header*)(&__buf ## a[0]))->inlinealloc = 1;														\
	a = (a_type*)(((stbds_array_header*)(&__buf ## a[0])) + 1)

// Allow a single inline storage buffer to be used for multiple sub-allocations.  Trims current allocation to what's used and returns a new allocation with the remainder if enough capacity exists.
#define stbds_arrinline_suballoc(a, a_type, min_capacity) \
	a = (a_type*)stbds_arrinline_suballocf(a, min_capacity)

// Inline simplified version of above that only works on character type.  Avoids elemsize multiply/divide math and call overhead.
#define stbds_arrinline_suballoc_char(a, min_capacity) \
	if (a && stbds_header(a)->inlinealloc)														\
	{																							\
		stbds_array_header* h = stbds_header(a);												\
		char* used_end = a + ((h->length + sizeof(size_t) - 1) & ~(sizeof(size_t) - 1));		\
		char* capacity_end = a + h->capacity;													\
		h->capacity = h->length;																\
		if (capacity_end - used_end < sizeof(stbds_array_header) + min_capacity)				\
		{																						\
			a = 0;																				\
		}																						\
		else																					\
		{																						\
			stbds_array_header* h_suballoc = (stbds_array_header*)used_end;						\
			h_suballoc->length = 0;																\
			h_suballoc->capacity = ((capacity_end - used_end) - sizeof(stbds_array_header));	\
			h_suballoc->hash_table = 0;															\
			h_suballoc->temp = 0;																\
			h_suballoc->elemsize = 1;															\
			h_suballoc->inlinealloc = 1;														\
			a = (char*)(h_suballoc + 1);														\
		}}   (void)0


#define stbds_arrsetcap(a, n) ((!(a) || (n) > stbds_header(a)->capacity) ? (stbds_arrgrow(a, 0, n), 0) : 0)
#define stbds_arrsetlen(a, n) ((stbds_arrcap(a) < (size_t)(n) ? stbds_arrsetcap((a), (size_t)(n)), 0 : 0), (a) ? stbds_header(a)->length = (size_t)(n) : 0)
#define stbds_arrsetlennocap(a, n) (stbds_header(a)->length = n)
#define stbds_arrinitlen(a, n) ((stbds_arrsetcap((a), (size_t)(n)), stbds_header(a)->length = (size_t)(n)))
#define stbds_arrcap(a) ((a) ? stbds_header(a)->capacity : 0)
#define stbds_arrcapnonull(a) (stbds_header(a)->capacity)
#define stbds_arrlen(a) ((a) ? (ptrdiff_t)stbds_header(a)->length : 0)
#define stbds_arrlennonull(a) ((ptrdiff_t)stbds_header(a)->length)
#define stbds_arrlenu(a) ((a) ? stbds_header(a)->length : 0)
#define stbds_arrput(a, v) (stbds_arrmaybegrow(a, 1), (a)[stbds_header(a)->length++] = (v))
#define stbds_arrpush stbds_arrput	// synonym
#define stbds_arrpop(a) (stbds_header(a)->length--, (a)[stbds_header(a)->length])
#define stbds_arraddn(a, n) ((void)(stbds_arraddnoff(a, n)))  // deprecated, use one of the following instead:
#define stbds_arraddnptr(a, n) (stbds_arrmaybegrow(a, n), stbds_header(a)->length += (n), &(a)[stbds_header(a)->length - (n)])
#define stbds_arraddnoff(a, n) (stbds_arrmaybegrow(a, n), stbds_header(a)->length += (n), stbds_header(a)->length - (n))
#define stbds_arraddnindex(a, n) stbds_arraddnoff(a, n)
#define stbds_arrlast(a) ((a)[stbds_header(a)->length - 1])
#define stbds_arrfree(a) ((void)((a) && !stbds_header(a)->inlinealloc ? STB_COMMON_FREE(stbds_header(a)) : (void)0), (a) = NULL)
#define stbds_arrdel(a, i) stbds_arrdeln(a, i, 1)
#define stbds_arrdeln(a, i, n) (memmove(&(a)[i], &(a)[(i) + (n)], sizeof *(a) * (stbds_header(a)->length - (n) - (i))), stbds_header(a)->length -= (n))
#define stbds_arrdelswap(a, i) ((a)[i] = stbds_arrlast(a), stbds_header(a)->length -= 1)
#define stbds_arrinsn(a, i, n)                               \
	((a) ? stbds_temp(a) = (i) : 0, stbds_arraddn((a), (n)), \
	 memmove(&(a)[stbds_temp(a) + (n)], &(a)[stbds_temp(a)], sizeof *(a) * (stbds_header(a)->length - (n)-stbds_temp(a))))
#define stbds_arrins(a, i, v) (stbds_arrinsn((a), (i), 1), (a)[i] = (v))

#define stbds_arrmaybegrow(a, n) ((!(a) || stbds_header(a)->length + (n) > stbds_header(a)->capacity) ? (stbds_arrgrow(a, n, 0), 0) : 0)

#define stbds_arrgrow(a, b, c) ((a) = stbds_arrgrowf_wrapper((a), sizeof *(a), (b), (c)))

#define stbds_hmput(t, k, v)                                                                                                                      \
	((t) = stbds_hmput_key_wrapper((t), sizeof *(t), (void*)STBDS_ADDRESSOF((t)->key, (k)), sizeof(t)->key, 0), (t)[stbds_temp((t)-1)].key = (k), \
	 (t)[stbds_temp((t)-1)].value = (v))

#define stbds_hmputs(t, s) ((t) = stbds_hmput_key_wrapper((t), sizeof *(t), &(s).key, sizeof(s).key, STBDS_HM_BINARY), (t)[stbds_temp((t)-1)] = (s))

#define stbds_hmgeti(t, k) \
	((t) = stbds_hmget_key_wrapper((t), sizeof *(t), (void*)STBDS_ADDRESSOF((t)->key, (k)), sizeof(t)->key, STBDS_HM_BINARY), stbds_temp((t)-1))

#define stbds_hmgeti_ts(t, k, temp) \
	((t) = stbds_hmget_key_ts_wrapper((t), sizeof *(t), (void*)STBDS_ADDRESSOF((t)->key, (k)), sizeof(t)->key, &(temp), STBDS_HM_BINARY), (temp))

#define stbds_hmgetp(t, k) ((void)stbds_hmgeti(t, k), &(t)[stbds_temp((t)-1)])

#define stbds_hmgetp_ts(t, k, temp) ((void)stbds_hmgeti_ts(t, k, temp), &(t)[temp])

#define stbds_hmdel(t, k)                                                                                                                                 \
	(((t) = stbds_hmdel_key_wrapper((t), sizeof *(t), (void*)STBDS_ADDRESSOF((t)->key, (k)), sizeof(t)->key, STBDS_OFFSETOF((t), key), STBDS_HM_BINARY)), \
	 (t) ? stbds_temp((t)-1) : 0)

#define stbds_hmdefault(t, v) ((t) = stbds_hmput_default_wrapper((t), sizeof *(t)), (t)[-1].value = (v))

#define stbds_hmdefaults(t, s) ((t) = stbds_hmput_default_wrapper((t), sizeof *(t)), (t)[-1] = (s))

#define stbds_hmfree(p) ((void)((p) != NULL ? stbds_hmfree_func((p)-1, sizeof *(p)), 0 : 0), (p) = NULL)

#define stbds_hmgets(t, k) (*stbds_hmgetp(t, k))
#define stbds_hmget(t, k) (stbds_hmgetp(t, k)->value)
#define stbds_hmget_ts(t, k, temp) (stbds_hmgetp_ts(t, k, temp)->value)
#define stbds_hmlen(t) ((t) ? (ptrdiff_t)stbds_header((t)-1)->length - 1 : 0)
#define stbds_hmlenu(t) ((t) ? stbds_header((t)-1)->length - 1 : 0)
#define stbds_hmgetp_null(t, k) (stbds_hmgeti(t, k) == -1 ? NULL : &(t)[stbds_temp((t)-1)])

#define stbds_shput(t, k, v) ((t) = stbds_hmput_key_wrapper((t), sizeof *(t), (void*)(k), sizeof(t)->key, STBDS_HM_STRING), (t)[stbds_temp((t)-1)].value = (v))

#define stbds_shputi(t, k, v) \
	((t) = stbds_hmput_key_wrapper((t), sizeof *(t), (void*)(k), sizeof(t)->key, STBDS_HM_STRING), (t)[stbds_temp((t)-1)].value = (v), stbds_temp((t)-1))

#define stbds_shputs(t, s)                                                                                                          \
	((t) = stbds_hmput_key_wrapper((t), sizeof *(t), (void*)(s).key, sizeof(s).key, STBDS_HM_STRING), (t)[stbds_temp((t)-1)] = (s), \
	 (t)[stbds_temp((t)-1)].key = stbds_temp_key((t)-1))  // above line overwrites whole structure, so must rewrite key here if it was allocated internally

#define stbds_pshput(t, p) \
	((t) = stbds_hmput_key_wrapper((t), sizeof *(t), (void*)(p)->key, sizeof(p)->key, STBDS_HM_PTR_TO_STRING), (t)[stbds_temp((t)-1)] = (p))

#define stbds_shgeti(t, k) ((t) = stbds_hmget_key_wrapper((t), sizeof *(t), (void*)(k), sizeof(t)->key, STBDS_HM_STRING), stbds_temp((t)-1))

#define stbds_pshgeti(t, k) ((t) = stbds_hmget_key_wrapper((t), sizeof *(t), (void*)(k), sizeof(*(t))->key, STBDS_HM_PTR_TO_STRING), stbds_temp((t)-1))

#define stbds_shgetp(t, k) ((void)stbds_shgeti(t, k), &(t)[stbds_temp((t)-1)])

#define stbds_pshget(t, k) ((void)stbds_pshgeti(t, k), (t)[stbds_temp((t)-1)])

#define stbds_shdel(t, k) \
	(((t) = stbds_hmdel_key_wrapper((t), sizeof *(t), (void*)(k), sizeof(t)->key, STBDS_OFFSETOF((t), key), STBDS_HM_STRING)), (t) ? stbds_temp((t)-1) : 0)
#define stbds_pshdel(t, k)                                                                                                                \
	(((t) = stbds_hmdel_key_wrapper((t), sizeof *(t), (void*)(k), sizeof(*(t))->key, STBDS_OFFSETOF(*(t), key), STBDS_HM_PTR_TO_STRING)), \
	 (t) ? stbds_temp((t)-1) : 0)

#define stbds_sh_new_arena(t) ((t) = stbds_shmode_func_wrapper(t, sizeof *(t), STBDS_SH_ARENA))
#define stbds_sh_new_strdup(t) ((t) = stbds_shmode_func_wrapper(t, sizeof *(t), STBDS_SH_STRDUP))

#define stbds_shdefault(t, v) stbds_hmdefault(t, v)
#define stbds_shdefaults(t, s) stbds_hmdefaults(t, s)

#define stbds_shfree stbds_hmfree
#define stbds_shlenu stbds_hmlenu

#define stbds_shgets(t, k) (*stbds_shgetp(t, k))
#define stbds_shget(t, k) (stbds_shgetp(t, k)->value)
#define stbds_shgetp_null(t, k) (stbds_shgeti(t, k) == -1 ? NULL : &(t)[stbds_temp((t)-1)])
#define stbds_shlen stbds_hmlen

typedef struct
{
	int elemsize;
	int inlinealloc;		// Allocated inline, don't free!
	void* hash_table;
	ptrdiff_t temp;
	int capacity;
	int length;
} stbds_array_header;

#ifdef _MSC_VER
#pragma warning(push)
#pragma warning(disable: 4200)
#endif

typedef struct stbds_string_block
{
	struct stbds_string_block* next;
	char storage[];
} stbds_string_block;

#ifdef _MSC_VER
#pragma warning(pop)
#endif

struct stbds_string_arena
{
	stbds_string_block* storage;
	size_t remaining;
	unsigned char block;
	unsigned char mode;	 // this isn't used by the string arena itself
};

#define STBDS_HM_BINARY 0
#define STBDS_HM_STRING 1

enum
{
	STBDS_SH_NONE,
	STBDS_SH_DEFAULT,
	STBDS_SH_STRDUP,
	STBDS_SH_ARENA
};

#ifdef __cplusplus
// in C we use implicit assignment from these void*-returning functions to T*.
// in C++ these templates make the same code work
template <class T>
static T* stbds_arrgrowf_wrapper(T* a, size_t elemsize, size_t addlen, size_t min_cap)
{
	return (T*)stbds_arrgrowf((void*)a, elemsize, addlen, min_cap);
}
template <class T>
static T* stbds_hmget_key_wrapper(T* a, size_t elemsize, void* key, size_t keysize, int mode)
{
	return (T*)stbds_hmget_key((void*)a, elemsize, key, keysize, mode);
}
template <class T>
static T* stbds_hmget_key_ts_wrapper(T* a, size_t elemsize, void* key, size_t keysize, ptrdiff_t* temp, int mode)
{
	return (T*)stbds_hmget_key_ts((void*)a, elemsize, key, keysize, temp, mode);
}
template <class T>
static T* stbds_hmput_default_wrapper(T* a, size_t elemsize)
{
	return (T*)stbds_hmput_default((void*)a, elemsize);
}
template <class T>
static T* stbds_hmput_key_wrapper(T* a, size_t elemsize, void* key, size_t keysize, int mode)
{
	return (T*)stbds_hmput_key((void*)a, elemsize, key, keysize, mode);
}
template <class T>
static T* stbds_hmdel_key_wrapper(T* a, size_t elemsize, void* key, size_t keysize, size_t keyoffset, int mode)
{
	return (T*)stbds_hmdel_key((void*)a, elemsize, key, keysize, keyoffset, mode);
}
template <class T>
static T* stbds_shmode_func_wrapper(T*, size_t elemsize, int mode)
{
	return (T*)stbds_shmode_func(elemsize, mode);
}
#else
#define stbds_arrgrowf_wrapper stbds_arrgrowf
#define stbds_hmget_key_wrapper stbds_hmget_key
#define stbds_hmget_key_ts_wrapper stbds_hmget_key_ts
#define stbds_hmput_default_wrapper stbds_hmput_default
#define stbds_hmput_key_wrapper stbds_hmput_key
#define stbds_hmdel_key_wrapper stbds_hmdel_key
#define stbds_shmode_func_wrapper(t, e, m) stbds_shmode_func(e, m)
#endif


/*
------------------------------------------------------------------------------
This software is available under 2 licenses -- choose whichever you prefer.
------------------------------------------------------------------------------
ALTERNATIVE A - MIT License
Copyright (c) 2019 Sean Barrett
Permission is hereby granted, free of charge, to any person obtaining a copy of
this software and associated documentation files (the "Software"), to deal in
the Software without restriction, including without limitation the rights to
use, copy, modify, merge, publish, distribute, sublicense, and/or sell copies
of the Software, and to permit persons to whom the Software is furnished to do
so, subject to the following conditions:
The above copyright notice and this permission notice shall be included in all
copies or substantial portions of the Software.
THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND, EXPRESS OR
IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF MERCHANTABILITY,
FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT. IN NO EVENT SHALL THE
AUTHORS OR COPYRIGHT HOLDERS BE LIABLE FOR ANY CLAIM, DAMAGES OR OTHER
LIABILITY, WHETHER IN AN ACTION OF CONTRACT, TORT OR OTHERWISE, ARISING FROM,
OUT OF OR IN CONNECTION WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN THE
SOFTWARE.
------------------------------------------------------------------------------
ALTERNATIVE B - Public Domain (www.unlicense.org)
This is free and unencumbered software released into the public domain.
Anyone is free to copy, modify, publish, use, compile, sell, or distribute this
software, either in source code form or as a compiled binary, for any purpose,
commercial or non-commercial, and by any means.
In jurisdictions that recognize copyright laws, the author or authors of this
software dedicate any and all copyright interest in the software to the public
domain. We make this dedication for the benefit of the public at large and to
the detriment of our heirs and successors. We intend this dedication to be an
overt act of relinquishment in perpetuity of all present and future rights to
this software under copyright law.
THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND, EXPRESS OR
IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF MERCHANTABILITY,
FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT. IN NO EVENT SHALL THE
AUTHORS BE LIABLE FOR ANY CLAIM, DAMAGES OR OTHER LIABILITY, WHETHER IN AN
ACTION OF CONTRACT, TORT OR OTHERWISE, ARISING FROM, OUT OF OR IN CONNECTION
WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN THE SOFTWARE.
------------------------------------------------------------------------------
*/
