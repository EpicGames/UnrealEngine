// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "ShaderCore.h"

class FShaderCompilerDefinitions;
class FShaderPreprocessOutput;
class FString;
struct FShaderCompilerInput;
struct FShaderCompilerOutput;

/**
 * Preprocess a shader.
 * @param Output - Preprocess output struct. Source, directives and possibly errors will be populated.
 * @param Input - The shader compiler input.
 * @param MergedEnvironment - The result of merging the Environment and SharedEnvironment from the FShaderCompilerInput
 * (it is assumed this overload is called outside of the worker process which merges this in-place, so this merge step must be
 * performed by the caller)
 * @param AdditionalDefines - Additional defines with which to preprocess the shader.
 * @returns true if the shader is preprocessed without error.
 */
extern SHADERPREPROCESSOR_API bool PreprocessShader(
	FShaderPreprocessOutput& Output,
	const FShaderCompilerInput& Input,
	const FShaderCompilerEnvironment& MergedEnvironment,
	const FShaderCompilerDefinitions& AdditionalDefines);

/**
 * Preprocess a shader.
 * @param Output - Preprocess output struct. Source, directives and possibly errors will be populated.
 * @param Input - The shader compiler input.
 * @param MergedEnvironment - The result of merging the Environment and SharedEnvironment from the FShaderCompilerInput
 * (it is assumed this overload is called outside of the worker process which merges this in-place, so this merge step must be
 * performed by the caller)
 * @returns true if the shader is preprocessed without error.
 */
extern SHADERPREPROCESSOR_API bool PreprocessShader(
	FShaderPreprocessOutput& Output,
	const FShaderCompilerInput& Input,
	const FShaderCompilerEnvironment& MergedEnvironment);
