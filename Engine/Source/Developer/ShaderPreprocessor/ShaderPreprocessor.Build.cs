// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ShaderPreprocessor : ModuleRules
{
	public ShaderPreprocessor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"RenderCore",
			}
			);

		PrivateDefinitions.Add("STB_CONFIG=../StbConfig.h");

		// TODO: Disable SN-DBS distribution until ../StbConfig.h include from STB_CONFIG can be resolved correctly
		bBuildLocallyWithSNDBS = true;
	}
}
