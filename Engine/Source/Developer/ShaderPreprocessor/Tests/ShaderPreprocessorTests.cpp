// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "stb_preprocess/cond_expr.h"
#include "stb_preprocess/preprocessor.h"
#include "ShaderSource.h"
#include "TestHarness.h"

int TestEvaluate(const char* Expression, int& Result)
{
	return evaluate_integer_constant_expression(const_cast<char*>(Expression), &Result);
}

TEST_CASE("Shaders::ShaderPreprocessor::ConditionalExpression", "[EditorContext][EngineFilter]")
{
	int Result;
	CHECK(TestEvaluate("4+3*2", Result) == 10);
	CHECK(Result == CE_RESULT_ok);
	CHECK(TestEvaluate("4+3*2==10", Result) == 1);
	CHECK(Result == CE_RESULT_ok);
	CHECK(TestEvaluate("4-3*2", Result) == -2);
	CHECK(Result == CE_RESULT_ok);
	CHECK(TestEvaluate("-4/2u", Result) == -2);
	CHECK(Result == CE_RESULT_ok);
	CHECK(TestEvaluate("4u-5", Result) == -1);
	CHECK(Result == CE_RESULT_ok);
	CHECK(TestEvaluate("4U-5", Result) == -1);
	CHECK(Result == CE_RESULT_ok);
	TestEvaluate("10000000000000000000000", Result);
	CHECK(Result == CE_RESULT_overflow);
	TestEvaluate("0xffffffff+1", Result);
	CHECK(Result == CE_RESULT_overflow);
	TestEvaluate("0xffffffff*2", Result);
	CHECK(Result == CE_RESULT_overflow);
	TestEvaluate("-2147483647 - 2", Result);
	CHECK(Result == CE_RESULT_overflow);
	TestEvaluate("1+=1", Result);
	CHECK(Result == CE_RESULT_syntax_error);
	TestEvaluate("1+(", Result);
	CHECK(Result == CE_RESULT_syntax_error);
	TestEvaluate("(1+", Result);
	CHECK(Result == CE_RESULT_syntax_error);
	CHECK(TestEvaluate("2/1", Result) == 2);
	CHECK(Result == CE_RESULT_ok);
	TestEvaluate("1/0", Result);
	CHECK(Result == CE_RESULT_division_by_zero);
	TestEvaluate("1 || 1/0", Result);
	CHECK(Result == CE_RESULT_ok);
	TestEvaluate("0 || 1/0", Result);
	CHECK(Result == CE_RESULT_division_by_zero);
	TestEvaluate("1 && 1/0", Result);
	CHECK(Result == CE_RESULT_division_by_zero);
	TestEvaluate("0 && 1/0", Result);
	CHECK(Result == CE_RESULT_ok);
	TestEvaluate("0 ? 1/0 : 2/1", Result);
	CHECK(Result == CE_RESULT_ok);
	TestEvaluate("1 ? 1/0 : 2/1", Result);
	CHECK(Result == CE_RESULT_division_by_zero);
	TestEvaluate("0 ? 2/1 : 1/0", Result);
	CHECK(Result == CE_RESULT_division_by_zero);
	TestEvaluate("1 ? 2/1 : 1/0", Result);
	CHECK(Result == CE_RESULT_ok);
}


const ANSICHAR* TestLoadFile(const ANSICHAR* Filename, void* RawContext, size_t* OutLength)
{
	const FShaderSource::FViewType* View = reinterpret_cast<const FShaderSource::FViewType*>(RawContext);
	*OutLength = View->Len();
	return View->GetData();
}

void TestFreeFile(const ANSICHAR* Filename, const ANSICHAR* Contents, void* RawContext)
{
	// noop, no files are loaded for these tests but preprocessor calls the freefile callback unconditionally
}

struct FPreprocessTestResult
{
	char* Source = nullptr;
	int NumDiagnostics = 0;
	pp_diagnostic* Diagnostics = nullptr;
	~FPreprocessTestResult()
	{
		preprocessor_file_free(Source, Diagnostics);
	}
};

FPreprocessTestResult ExecutePreprocessTest(const FShaderSource& Source)
{
	FPreprocessTestResult Result;
	static bool bInitialized = false;
	if (!bInitialized)
	{
		init_preprocessor(&TestLoadFile, &TestFreeFile, nullptr, nullptr, nullptr);
		bInitialized = true;
	}
	FShaderSource::FViewType SourceView = Source.GetView();
	Result.Source = preprocess_file("preprocessor_test", &SourceView, nullptr, 0u, &Result.Diagnostics, &Result.NumDiagnostics, nullptr, 0u);
	return Result;
}

TEST_CASE("Shaders::ShaderPreprocessor::ConditionalExpressionErrorReporting", "[EditorContext][EngineFilter]")
{
	// need this dance of multiple assembled string literal fragments to avoid the C++ preprocessor parsing directives inside the literal
	FShaderSource Source(ANSITEXTVIEW(
		"#" "if 0xFFFFFFFF+1 == 2\n"
		"#" "error \"Unreachable\n\""
		"#" "endif\n"));
	FPreprocessTestResult Result = ExecutePreprocessTest(Source);
	CHECK(Result.NumDiagnostics == 1);
	CHECK(Result.Diagnostics[0].where->line_number == 1);
	CHECK(strstr(Result.Diagnostics[0].message, "Overflow") != nullptr);
}

TEST_CASE("Shaders::ShaderPreprocessor::InvalidCharErrors", "[EditorContext][EngineFilter]")
{
	{
		// EOF after #define
		FShaderSource Source(ANSITEXTVIEW(
			"#" "define"));
		FPreprocessTestResult Result = ExecutePreprocessTest(Source);
		CHECK(Result.NumDiagnostics == 1);
		CHECK(Result.Diagnostics[0].where->line_number == 1);
		CHECK(strstr(Result.Diagnostics[0].message, "Invalid character EOF") != nullptr);
	}

	{
		// Invalid printable char in define param list
		FShaderSource Source(ANSITEXTVIEW(
			"#" "define BLAH(abc,#) abc\n"));
		FPreprocessTestResult Result = ExecutePreprocessTest(Source);
		CHECK(Result.NumDiagnostics == 1);
		CHECK(Result.Diagnostics[0].where->line_number == 1);
		CHECK(strstr(Result.Diagnostics[0].message, "Invalid character '#'") != nullptr);
	}

	{
		// Invalid non-printable char in define param list
		FShaderSource Source(ANSITEXTVIEW(
			"#" "define BLAH(abc,\3) abc\n"));
		FPreprocessTestResult Result = ExecutePreprocessTest(Source);
		CHECK(Result.NumDiagnostics == 1);
		CHECK(Result.Diagnostics[0].where->line_number == 1);
		CHECK(strstr(Result.Diagnostics[0].message, "Invalid character value 3") != nullptr);
	}

	{
		// Unexpected char after #if defined(
		FShaderSource Source(ANSITEXTVIEW(
			"#" "define BLAH 1\n"
			"#" "if defined(BLAH*)\n"
			"#" "endif\n"
		));
		FPreprocessTestResult Result = ExecutePreprocessTest(Source);
		CHECK(Result.NumDiagnostics == 1);
		CHECK(Result.Diagnostics[0].where->line_number == 2);
		CHECK(strstr(Result.Diagnostics[0].message, "unexpected character '*'") != nullptr);
	}
}

TEST_CASE("Shaders::ShaderPreprocessor::BuiltinMacros", "[EditorContext][EngineFilter]")
{
	FShaderSource Source(ANSITEXTVIEW(
		"int A = __" "LINE__;\n"
		"int B = __" "COUNTER__;\n"
		"int C = __" "COUNTER__;\n"
	));
	FPreprocessTestResult Result = ExecutePreprocessTest(Source);
	CHECK(Result.NumDiagnostics == 0);
	CHECK(strstr(Result.Source, "int A = 1") != nullptr);
	CHECK(strstr(Result.Source, "int B = 0") != nullptr);
	CHECK(strstr(Result.Source, "int C = 1") != nullptr);
}

TEST_CASE("Shaders::ShaderPreprocessor::EmitLineDirective", "[EditorContext][EngineFilter]")
{
	// line directive will be emitted if 12 or more lines are removed due to an inactive preprocessor block
	// (otherwise the inactive code will be replaced with empty lines)
	FShaderSource Source(ANSITEXTVIEW(
		"#" "define MYDIR 0\n"
		"#" "if MYDIR\n"
		"int A = 0;\n"
		"#" "endif\n"
		"#" "if MYDIR\n"
		"int B = 0;\n"
		"B++;\n"
		"B++;\n"
		"B++;\n"
		"B++;\n"
		"B++;\n"
		"B++;\n"
		"B++;\n"
		"B++;\n"
		"B++;\n"
		"B++;\n"
		"B++;\n"
		"#" "endif\n"
		"int C = 0;\n"
	));

	// exactly two line directives will be emitted, one on the first line (done for all files) and one after the second inactive block (starting with the newline on line 18)
	FPreprocessTestResult Result = ExecutePreprocessTest(Source);
	CHECK(Result.NumDiagnostics == 0);
	ANSICHAR* Directive = strstr(Result.Source, "#line");
	if (Directive)
	{
		CHECK(strstr(Directive, "1 \"preprocessor_test\"") != nullptr);
		ANSICHAR* NextDirective = strstr(Directive + 5, "#line");
		if (NextDirective)
		{
			CHECK(strstr(NextDirective, "18 \"preprocessor_test\"") != nullptr);
		}
		else
		{
			FAIL_CHECK("No second line directive found");
		}
	}
	else
	{
		FAIL_CHECK("No line directive found");
	}
}

#endif
